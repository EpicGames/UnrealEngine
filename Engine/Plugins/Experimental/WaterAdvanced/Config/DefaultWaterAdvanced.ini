[CoreRedirects]
+PropertyRedirects=(OldName="/Script/WaterAdvanced.ShallowWaterSettings.PhysicsAssetOverridesDataAsset",NewName="/Script/WaterAdvanced.ShallowWaterSettings.PhysicsAssetProxiesDataAsset")
+StructRedirects=(OldName="/Script/WaterAdvanced.ShallowWaterCollisionAgent",NewName="/Script/WaterAdvanced.ShallowWaterCollisionTracker_Actor")
+FunctionRedirects=(OldName="/Script/WaterAdvanced.ShallowWaterSubsystem.AddCollisionAgent",NewName="/Script/WaterAdvanced.ShallowWaterSubsystem.AddCollisionTrackerForActor")
+FunctionRedirects=(OldName="/Script/WaterAdvanced.ShallowWaterSubsystem.RemoveCollisionAgent",NewName="/Script/WaterAdvanced.ShallowWaterSubsystem.RemoveCollisionAgentForActor")
+FunctionRedirects=(OldName="/Script/WaterAdvanced.ShallowWaterSubsystem.RemoveCollisionAgentForActor",NewName="/Script/WaterAdvanced.ShallowWaterSubsystem.RemoveCollisionTrackerForActor")