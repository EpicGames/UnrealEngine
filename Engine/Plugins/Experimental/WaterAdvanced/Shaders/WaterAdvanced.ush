// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
WaterAdvanced.ush 
=============================================================================*/

#pragma once

static const float TWOPI = 2. * PI;

void BracketAngle(float2 Velocity, float BracketAngleDelta, out float2 VelocityDir0, out float2 VelocityDir1, out float BlendAlpha)
{
	if (BracketAngleDelta > 1e-5)
	{
		const float AngleDelta = BracketAngleDelta * PI / 180.0;
		const float2x2 RotateByAngleDelta = float2x2(cos(AngleDelta), sin(AngleDelta), -sin(AngleDelta), cos(AngleDelta));

		// direction of input velocity		
		float Angle = atan2(Velocity.y, Velocity.x);
		Angle = Angle < 0.0 ? Angle + 2.0 * PI : Angle;

		// Snap to min bracketed angle
		const float Fractional = fmod(Angle, AngleDelta);
		const float Angle0 = Angle - Fractional;
    
		// Compute velocity direction snapped to the bracketed angle
		VelocityDir0 = float2(cos(Angle0), sin(Angle0));

		// second bracketed direction is just rotated by the angle delta
		VelocityDir1 = mul(VelocityDir0, RotateByAngleDelta);

		// blend is the distance the velocity heading is to the first bracketed angle
		BlendAlpha = Fractional / AngleDelta;
	}
	else
	{
		// edge case where angle delta is 0, this turns off bracketing
		VelocityDir0 = normalize(Velocity);
		VelocityDir1 = VelocityDir0;		
		BlendAlpha = 0.0;
	}
}

  // More general form of bracketing - takes just a float and a bracket size.
void BracketValue(float Value, float BracketMagnitudeDelta, out float Value0, out float Value1, out float BlendAlpha)
{	
	if (BracketMagnitudeDelta > 1e-5)
	{
		const float Fractional = fmod(Value, BracketMagnitudeDelta);
		Value0 = Value - Fractional;
		Value1 = Value0 + BracketMagnitudeDelta;
		BlendAlpha = Fractional / BracketMagnitudeDelta;
	}
	else
	{
		// edge case where magnitude delta is 0, this turns off bracketing
		Value0 = Value;
		Value1 = Value;
		BlendAlpha = 0.;
	}
}

float2 CalculateUV(float2 vAxis, float2 position, float Time, float Speed)
{
	// second 2d vector perpendicular to the vAxis passed in
	const float2 uAxis = float2(-vAxis.y, vAxis.x);
	
	// project the position onto the basis vectors
	float2 uv = float2(dot(position, uAxis), dot(position, vAxis));
    
	// animated along one of the basis axes
	uv.y -= Time * Speed;
	
	return uv;
}

float ComputeMipLevel(float2 uv, int2 sz)
{	
	const float2 dxval = ddx(uv) * sz;
	const float2 dyval = ddy(uv) * sz;

	const float DDXLengthSquared = dot(dxval, dxval);
	const float DDYLengthSquared = dot(dyval, dyval);
	const float MaxLengthSquared = max(DDXLengthSquared, DDYLengthSquared);
	
	return 0.5f * log2(max(MaxLengthSquared, 1e-8f));
}

void Compute3Phases(const float PulseLength, const float Time, 
	out float lerp0, out float lerp1, out float lerp2,
	out float t0, out float t1, out float t2, out float3 CycleCount)
{	
	const float TimeScale = TWOPI / PulseLength;

	const int NumPhases = 3;

	const float lerp_t0 = TimeScale * Time + 0. * TWOPI / NumPhases;
	const float lerp_t1 = TimeScale * Time + 1. * TWOPI / NumPhases;
	const float lerp_t2 = TimeScale * Time + 2. * TWOPI / NumPhases;

	lerp0 = (.5 - .5 * cos(lerp_t0)) / 1.5;
	lerp1 = (.5 - .5 * cos(lerp_t1)) / 1.5;
	lerp2 = (.5 - .5 * cos(lerp_t2)) / 1.5;

	t0 = (Time + PulseLength * 0. / NumPhases) % PulseLength;
	t1 = (Time + PulseLength * 1. / NumPhases) % PulseLength;
	t2 = (Time + PulseLength * 2. / NumPhases) % PulseLength;

	CycleCount = float3(
		floor((Time + PulseLength * 0. / NumPhases) / PulseLength), 
		floor((Time + PulseLength * 1. / NumPhases) / PulseLength),
		floor((Time + PulseLength * 2. / NumPhases) / PulseLength));
}

void Compute2Phases(const float PulseLength, const float Time,
	out float lerpV,
	out float t0, out float t1, out float2 CycleCount)
{	
	const float HalfPulseLength = .5 * PulseLength;
	t0 = Time % PulseLength;
	t1 = (Time + HalfPulseLength) % PulseLength;

	lerpV = -abs(t1 * 2. / PulseLength - 1.) + 1.;
	
	CycleCount = float2(floor(Time / PulseLength), floor((Time + HalfPulseLength) / PulseLength));
}

float4 AdvectionInterpolation2(float2 UV, float2 Velocity, float t0, float t1, float lerpV,
  float AdvectedTextureScale, float2 TexMipLevel, Texture2D<float4> Tex, SamplerState TexSampler, float2 CycleCount)
{
	// advect uv, offset position somewhat randomly by the cycle count
	float OffsetRandX = PseudoRandom(float2(CycleCount.x, 0))*1024.;
	float OffsetRandY = PseudoRandom(float2(CycleCount.y, 0))*1024.;

	const float2 AdvectedUV0 = UV - Velocity * t0 + OffsetRandX;
	const float2 AdvectedUV1 = UV - Velocity * t1 + OffsetRandY;

	// scale uv
	const float2 uv0 = AdvectedUV0 * AdvectedTextureScale;
	const float2 uv1 = AdvectedUV1 * AdvectedTextureScale;

	// sample the texture for the two phases
	const float4 TextureSample0 = Tex.SampleLevel(TexSampler, uv0, TexMipLevel.x);
	const float4 TextureSample1 = Tex.SampleLevel(TexSampler, uv1, TexMipLevel.y);

	// lerp according to phases
	return lerp(TextureSample0, TextureSample1, lerpV);	
}

float4 AdvectionInterpolation3(float2 UV, float2 Velocity, float t0, float t1, float t2, float lerp0, float lerp1, float lerp2,
  float AdvectedTextureScale, float3 TexMipLevel, Texture2D<float4> Tex, SamplerState TexSampler, float3 CycleCount)
{
	// advect uvs
	float OffsetRandX = PseudoRandom(float2(CycleCount.x, 0))*1024.;
	float OffsetRandY = PseudoRandom(float2(CycleCount.y, 0))*1024.;
	float OffsetRandZ = PseudoRandom(float2(CycleCount.z, 0))*1024.;

	const float2 AdvectedUV0 = UV - Velocity * t0 + OffsetRandX;
	const float2 AdvectedUV1 = UV - Velocity * t1 + OffsetRandY;
	const float2 AdvectedUV2 = UV - Velocity * t2 + OffsetRandZ;

	// scale uvs
	const float2 uv0 = AdvectedUV0 * AdvectedTextureScale;
	const float2 uv1 = AdvectedUV1 * AdvectedTextureScale;
	const float2 uv2 = AdvectedUV2 * AdvectedTextureScale;
		
	// sample texture for each phase
	const float4 TextureSample0 = Tex.SampleLevel(TexSampler, uv0, TexMipLevel.x);
	const float4 TextureSample1 = Tex.SampleLevel(TexSampler, uv1, TexMipLevel.y);
	const float4 TextureSample2 = Tex.SampleLevel(TexSampler, uv2, TexMipLevel.z);
	
	// blend according to phase
	return TextureSample0 * lerp0 + TextureSample1 * lerp1 + TextureSample2 * lerp2;
}