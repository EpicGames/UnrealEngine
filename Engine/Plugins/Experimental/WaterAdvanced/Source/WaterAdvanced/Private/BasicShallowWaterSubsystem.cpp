// Copyright Epic Games, Inc. All Rights Reserved.

#include "BasicShallowWaterSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicShallowWaterSubsystem)

UBasicShallowWaterSubsystem::UBasicShallowWaterSubsystem()
{
}

bool UBasicShallowWaterSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	TObjectPtr<UShallowWaterSettings> TmpSettings = GetMutableDefault<UShallowWaterSettings>();
	if (TmpSettings && !TmpSettings->UseDefaultShallowWaterSubsystem)
	{
		return false;
	}
	else
	{
		return Super::ShouldCreateSubsystem(Outer);
	}	
}

bool UBasicShallowWaterSubsystem::IsShallowWaterAllowedToInitialize() const
{
	TObjectPtr<UShallowWaterSettings> TmpSettings = GetMutableDefault<UShallowWaterSettings>();
	if (TmpSettings)
	{
		return TmpSettings->UseDefaultShallowWaterSubsystem;
	}

	return false;
}
