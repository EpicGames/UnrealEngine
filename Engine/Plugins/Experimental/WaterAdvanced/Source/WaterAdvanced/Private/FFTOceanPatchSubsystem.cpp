// Copyright Epic Games, Inc. All Rights Reserved.

#include "FFTOceanPatchSubsystem.h"
#include "ShallowWaterSettings.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/World.h"
#include "Engine/AssetManager.h"
#include "NiagaraComponent.h"
#include "NiagaraFunctionLibrary.h"
#include "ShallowWaterCommon.h"
#include "NiagaraComponent.h"
#include "Engine/TextureRenderTarget2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FFTOceanPatchSubsystem)

UFFTOceanPatchSubsystem::UFFTOceanPatchSubsystem()
{
}

bool UFFTOceanPatchSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!FApp::CanEverRender() || IsRunningDedicatedServer())
	{
		return false;
	}

	// IsRunningDedicatedServer() is a static check and doesn't work in PIE "As Client' mode where both a server and a client are run
	if (UWorld* World = Cast<UWorld>(Outer))
	{
		if (World->IsNetMode(NM_DedicatedServer))
		{
			return false;
		}
	}
	
	return Super::ShouldCreateSubsystem(Outer);	
}

void UFFTOceanPatchSubsystem::PostInitialize()
{
	Super::PostInitialize();

	Settings = GetMutableDefault<UShallowWaterSettings>();

	if (!Settings)
	{
		ensureMsgf(false, TEXT("UShallowWaterSubsystem::PostInitialize() - UShallowWaterSettings is not valid"));
		return;
	}

	TArray<FSoftObjectPath> ObjectsToLoad;
	ObjectsToLoad.Add(Settings->DefaultOceanPatchNiagaraSystem.ToSoftObjectPath());

	// To avoid warnings in cooked builds do a check with the asset registry that we cooked the assets
	if (FPlatformProperties::RequiresCookedData())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		for (auto It=ObjectsToLoad.CreateIterator(); It; ++It)
		{
			FAssetData AssetData;
			if (AssetRegistry.TryGetAssetByObjectPath(*It, AssetData) != UE::AssetRegistry::EExists::Exists)
			{
				It.RemoveCurrentSwap();
			}
		}
	}

	if (ObjectsToLoad.Num() > 0)
	{
		UAssetManager::GetStreamableManager().RequestAsyncLoad(
			ObjectsToLoad,
			FStreamableDelegate::CreateWeakLambda(this,
				[this]()
				{
					// continue with initialization		
				}
			)
		);	
	}

	FFTOceanSystem = nullptr;
	OceanNormalRT = nullptr;
}

TStatId UFFTOceanPatchSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UFFTOceanPatchSubsystem, STATGROUP_Tickables);
}

TObjectPtr<UTextureRenderTarget2D> UFFTOceanPatchSubsystem::GetOceanNormalRT(UWorld* World)
{
	if (FFTOceanSystem == nullptr)
	{
		Settings = GetMutableDefault<UShallowWaterSettings>();		
		UNiagaraSystem *NiagaraOceanSimulation = Settings->DefaultOceanPatchNiagaraSystem.Get();

		if (NiagaraOceanSimulation == nullptr)
		{
			UE_LOGF(LogShallowWater, Warning, "UFFTOceanPatchSubsystem::GetOceanNormalRT - Ocean simulation system not loaded");	
			return nullptr;
		}

		FFTOceanSystem = UNiagaraFunctionLibrary::SpawnSystemAtLocation(World, NiagaraOceanSimulation,
			FVector(0,0,0), FRotator::ZeroRotator, FVector::OneVector, false,
			false,
			ENCPoolMethod::None, false);
		
		if (FFTOceanSystem == nullptr)
		{
			UE_LOGF(LogShallowWater, Warning, "UFFTOceanPatchSubsystem::GetOceanNormalRT - Cannot spawn fft ocean system");	
			return nullptr;
		}

		if (World && World->bIsWorldInitialized)
		{
			if (!FFTOceanSystem->IsRegistered())
			{
				FFTOceanSystem->RegisterComponentWithWorld(World);
			}
		}
		else
		{
			UE_LOGF(LogShallowWater, Warning, "UFFTOceanPatchSubsystem::GetOceanNormalRT - World not initialized");	
			return nullptr;
		}

		FFTOceanSystem->SetVisibleFlag(true);
		FFTOceanSystem->SetAsset(NiagaraOceanSimulation);

		OceanNormalRT = NewObject<UTextureRenderTarget2D>(this, NAME_None, RF_Transient);
		OceanNormalRT->InitAutoFormat(1, 1);
		FFTOceanSystem->SetVariableTextureRenderTarget(FName("OceanNormalRT"), OceanNormalRT);
		FFTOceanSystem->Activate();		
	}

	return OceanNormalRT;
}
