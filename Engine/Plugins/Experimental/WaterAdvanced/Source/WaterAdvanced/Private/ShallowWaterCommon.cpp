// Copyright Epic Games, Inc. All Rights Reserved.


#include "ShallowWaterCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShallowWaterCommon)

DEFINE_LOG_CATEGORY(LogShallowWater);
