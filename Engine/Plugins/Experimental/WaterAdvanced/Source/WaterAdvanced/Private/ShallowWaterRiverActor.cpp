// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShallowWaterRiverActor.h"

#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraDataInterfaceArrayFunctionLibrary.h"

#include "WaterBodyActor.h"
#include "WaterBodyRiverComponent.h"
#include "WaterSplineComponent.h"
#include "WaterSubsystem.h"

#include "BakedShallowWaterSimulationComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/SplineMeshComponent.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/OverlapResult.h"

#include "TextureResource.h"
#include "ShallowWaterCommon.h"
#include "FFTOceanPatchSubsystem.h"
#include "Engine/World.h"
#include "Engine/Texture2D.h"
#include "Math/Float16Color.h"
#include "Kismet/GameplayStatics.h"
#include "Landscape.h"
#include "LandscapeStreamingProxy.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#include "EngineUtils.h"

#if WITH_EDITOR
#include "Editor.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(ShallowWaterRiverActor)

bool bShallowWaterRiverDebugVisualize = false;
FAutoConsoleVariableRef CVarShallowWaterRiverDebugVisualize(TEXT("r.ShallowWater.RiverDebugVisualize"), bShallowWaterRiverDebugVisualize, TEXT(""));

UShallowWaterRiverComponent::UShallowWaterRiverComponent(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	PrimaryComponentTick.bCanEverTick = true;	

#if WITH_EDITORONLY_DATA
	bTickInEditor = true;
#endif // WITH_EDITORONLY_DATA

	bIsInitialized = false;
	bTickInitialize = false;
	bRenderStateTickInitialize = false;

	ResolutionMaxAxis = 512;
	SourceSize = 1000;
	
	// Initialize landscape array with all landscapes.
	// Guard with IsInGameThread() because TActorIterator asserts game-thread access,
	// and this constructor can run on the async loading thread. During async loading
	// the serialized BottomContourLandscapeActors will be restored from disk anyway.
	if (IsInGameThread() && GetWorld())
	{
		for (TActorIterator<ALandscape> It(GetWorld(), ALandscape::StaticClass()); It; ++It)
		{
			BottomContourLandscapeActors.Add(*It);
		}

		for (TActorIterator<ALandscapeStreamingProxy> It(GetWorld(), ALandscapeStreamingProxy::StaticClass()); It; ++It)
		{
			BottomContourLandscapeActors.Add(*It);
		}
	}
}

TObjectPtr<UTextureRenderTarget2D> UShallowWaterRiverComponent::GetSharedFFTOceanPatchNormalRTFromSubsystem(UWorld* World)
{
	if (World != nullptr)
	{
		UFFTOceanPatchSubsystem *OceanPatchSubsystem = World->GetSubsystem<UFFTOceanPatchSubsystem>();

		if (OceanPatchSubsystem != nullptr)
		{
			return OceanPatchSubsystem->GetOceanNormalRT(World);
		}
		else
		{
			UE_LOGF(LogShallowWater, Warning, "No valid FFT ocean patch subsystem.");	
		}
	}
	else
	{
		UE_LOGF(LogShallowWater, Warning, "No valid World.");
	}

	return nullptr;
}

FBoxSphereBounds UShallowWaterRiverComponent::InitializeCaptureDI(const FName& DIName, TArray<AActor*> RawActorPtrArray)
{
	UNiagaraFunctionLibrary::SetSceneCapture2DDataInterfaceManagedMode(RiverSimSystem, DIName,
				ESceneCaptureSource::SCS_SceneDepth,
				FIntPoint(ResolutionMaxAxis, ResolutionMaxAxis),
				ETextureRenderTargetFormat::RTF_R32f,
				ECameraProjectionMode::Orthographic,
				90.0f,
				FMath::Max(WorldGridSize.X, WorldGridSize.Y),
				true,
				false,
				RawActorPtrArray,
				SceneCaptureLODDistanceFactor);

	// accumulate bounding box for river water bodies
	FBoxSphereBounds::Builder BottomContourCombinedWorldBoundsBuilder;
	for (AActor *BottomContourActor : RawActorPtrArray)
	{
		if (BottomContourActor != nullptr)
		{
			// accumulate bounds
			FBoxSphereBounds WorldBounds;
			BottomContourActor->GetActorBounds(false, WorldBounds.Origin, WorldBounds.BoxExtent);

			BottomContourCombinedWorldBoundsBuilder += WorldBounds;
		}
		else
		{
			UE_LOGF(LogShallowWater, Verbose, "UShallowWaterRiverComponent::Rebuild() - skipping null bottom contour boundary actor found");
			continue;
		}
	}
	return FBoxSphereBounds(BottomContourCombinedWorldBoundsBuilder);	
}

void UShallowWaterRiverComponent::ConvertToVirtualTextures()
{
#if WITH_EDITOR
	bool HasChanged = false;

	bUseVirtualTextures = true;

	if (BakedWaterSurfaceTexture != NULL && !BakedWaterSurfaceTexture->VirtualTextureStreaming && bUseVirtualTextures)
	{	
		SimRes = FVector2D(BakedWaterSurfaceTexture->Source.GetSizeX(), BakedWaterSurfaceTexture->Source.GetSizeY());
		RiverSimSystem->SetVariableVec2(FName("SimRes"), SimRes);

		InitializeVirtualTexture(BakedWaterSurfaceTexture);
		UE_LOGF(LogShallowWater, Warning, "Baked water surface texture was not virtual- converting.  Recommended resave.");

		HasChanged = true;
	}
	else if (BakedWaterSurfaceTexture != NULL && BakedWaterSurfaceTexture->VirtualTextureStreaming && 
		BakedWaterSurfaceTexture->PowerOfTwoMode != ETexturePowerOfTwoSetting::StretchToPowerOfTwo && bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedWaterSurfaceTexture);
		UE_LOGF(LogShallowWater, Warning, "Baked water surface texture wrong power of two mode - converting.  Recommended resave.");

		HasChanged = true;
	}
	
	if (BakedWaterSurfaceNormalTexture != NULL && !BakedWaterSurfaceNormalTexture->VirtualTextureStreaming && bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedWaterSurfaceNormalTexture);
		UE_LOGF(LogShallowWater, Warning, "Baked water surface normal texture was not virtual- converting.  Recommended resave.");

		HasChanged = true;
	}
	else if (BakedWaterSurfaceNormalTexture != NULL && BakedWaterSurfaceNormalTexture->VirtualTextureStreaming && 
		BakedWaterSurfaceNormalTexture->PowerOfTwoMode != ETexturePowerOfTwoSetting::StretchToPowerOfTwo && bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedWaterSurfaceNormalTexture);
		UE_LOGF(LogShallowWater, Warning, "Baked water surface normal texture wrong power of two mode - converting.  Recommended resave.");

		HasChanged = true;
	}

	if (BakedFoamTexture != NULL && !BakedFoamTexture->VirtualTextureStreaming && bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedFoamTexture);
		UE_LOGF(LogShallowWater, Warning, "Baked foam texture was not virtual- converting.  Recommended resave.");

		HasChanged = true;
	}
	else if (BakedFoamTexture != NULL && BakedFoamTexture->VirtualTextureStreaming && 
		BakedFoamTexture->PowerOfTwoMode != ETexturePowerOfTwoSetting::StretchToPowerOfTwo && bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedFoamTexture);
		UE_LOGF(LogShallowWater, Warning, "Baked water surface foam texture wrong power of two mode - converting.  Recommended resave.");

		HasChanged = true;
	}

	if (HasChanged)
	{
		PostEditChange();
	}
#endif
}

void UShallowWaterRiverComponent::PostLoad()
{
	Super::PostLoad();	


	// ensure all baked textures are virtual for backwards compatibility
	ConvertToVirtualTextures();

	if (RenderState == EShallowWaterRenderState::LiveSim || RiverSimSystem == nullptr)
	{
	#if WITH_EDITOR
		bIsInitialized = false;
		bTickInitialize = false;

		Rebuild();
	#endif
	}
	else
	{
		RiverSimSystem->ReinitializeSystem();
		RiverSimSystem->Activate();
	}

	bRenderStateTickInitialize = false;
}

void UShallowWaterRiverComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	// nothing to do on a server
	if (!FApp::CanEverRender() || IsRunningDedicatedServer())
	{
		return;
	}
	else if (UWorld* World = GetWorld()) // PIE server
	{
		if (World->IsNetMode(NM_DedicatedServer))
		{
			return;
		}
	}

#if WITH_EDITOR
	// lots of tick ordering issues, so we try to initialize on the first tick too
	if (!bTickInitialize && (RiverSimSystem == nullptr || (RenderState == EShallowWaterRenderState::LiveSim && !bIsInitialized)))
	{
		bTickInitialize = true;
		Rebuild();
	}
	else if (bIsInitialized)
	{
		if (RiverSimSystem)
		{
			RiverSimSystem->Activate();	
		}
		else
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::TickComponent() - null Niagara sim when trying to activate. Please reset.");
		}
	}
	else
	{
		// System is in a bad state
		// UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::TickComponent() - null Niagara sim when trying to activate. Please reset.");
	}
#endif

	if (!bRenderStateTickInitialize)
	{
		UpdateRenderState();
	}
}

void UShallowWaterRiverComponent::BeginPlay()
{
	Super::BeginPlay();

	bRenderStateTickInitialize = false;

	UpdateRenderState();

	// make sure the simulation is not going to be run in case of various initialization edge cases
	bool bReadBakedSim = RenderState == EShallowWaterRenderState::BakedSim || RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim || RenderState == EShallowWaterRenderState::WaterComponent;	
	if (RiverSimSystem != nullptr && bReadBakedSim)
	{
		RiverSimSystem->SetVariableBool(FName("ReadCachedSim"), bReadBakedSim);
		RiverSimSystem->ReinitializeSystem();
		RiverSimSystem->Activate();
	}
}

void UShallowWaterRiverComponent::OnUnregister()
{
	Super::OnUnregister();
}

#if WITH_EDITOR

void UShallowWaterRiverComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}
			
	// this should go before rebuild not after...something is wrong
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UShallowWaterRiverComponent, RenderState) && RiverSimSystem != nullptr && RiverSimSystem->IsActive())
	{
		bool bReadBakedSim = RenderState == EShallowWaterRenderState::BakedSim || RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim || RenderState == EShallowWaterRenderState::WaterComponent;	
		RiverSimSystem->SetVariableBool(FName("ReadCachedSim"), bReadBakedSim);
	}
	else
	{
		bIsInitialized = false;
		bTickInitialize = false;		
	}

	bRenderStateTickInitialize = false;

	Rebuild();
	UpdateRenderState();
	ReregisterComponent();

}

void UShallowWaterRiverComponent::Rebuild()
{	
	bIsInitialized = false;
	bTickInitialize = false;

	if (NiagaraRiverSimulation == nullptr)
	{
		NiagaraRiverSimulation = LoadObject<UNiagaraSystem>(nullptr, TEXT("/WaterAdvanced/Niagara/Systems/Grid2D_SW_River.Grid2D_SW_River"));
	}

	if (OceanPatchSystem == nullptr)
	{
		OceanPatchSystem = LoadObject<UNiagaraSystem>(nullptr, TEXT("/WaterAdvanced/Niagara/Systems/Grid2D_OceanPatch.Grid2D_OceanPatch"));
	}

	if (RiverSimSystem != nullptr)
	{
		RiverSimSystem->SetActive(false);
		RiverSimSystem->DestroyComponent();
		RiverSimSystem = nullptr;
	}
	
	if (ResolutionMaxAxis <= 0)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - resolution must be greater than 0");
		return;
	}

	if (NumSteps <= 0)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - num steps must be greater than 0");
		return;
	}

	if (SimSpeed <= 1e-8)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - speed must be greater than zero");
		return;
	}

	if (NiagaraRiverSimulation == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - null Niagara system asset");
		return;
	}

	AllWaterBodies.Empty();

	// collect all the water bodies	
	if (SourceRiverWaterBodies.Num() != 0)
	{
		for (TSoftObjectPtr<AWaterBody > CurrWaterBody : SourceRiverWaterBodies)
		{
			if (CurrWaterBody)
			{
				AllWaterBodies.Add(CurrWaterBody);
			}
			else 
			{
				UE_LOGF(LogShallowWater, Verbose, "UShallowWaterRiverComponent::Rebuild() - skipping null water body actor found");
				continue;
			}
		}
	}
	else	
	{
		UE_LOGF(LogShallowWater, Verbose, "UShallowWaterRiverComponent::Rebuild() - No source water bodies specified");
		return;
	}
	
	if (AllWaterBodies.Num() == 0)
	{
		UE_LOGF(LogShallowWater, Verbose, "UShallowWaterRiverComponent::Rebuild() - No valid source water bodies specified");
		return;
	}

	bool HasValidSinks = false;
	for (TSoftObjectPtr<AWaterBody> CurrWaterBody : SinkRiverWaterBodies)
	{
		if (CurrWaterBody != nullptr)
		{
			HasValidSinks = true;
			AllWaterBodies.Add(CurrWaterBody);			
		}
		else
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - skipping null sink water body actor found");
			continue;
		}
	}

	// flush all debug draw lines
#if ENABLE_DRAW_DEBUG
	FlushPersistentDebugLines(GetWorld());
#endif

	if (!HasValidSinks)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - no valid sinks, using the first source as a sink");
		SinkRiverWaterBodies.Add(*AllWaterBodies.CreateConstIterator());
	}

	// accumulate bounding box for river water bodies
	FBoxSphereBounds::Builder CombinedWorldBoundsBuilder;
	for (TSoftObjectPtr<AWaterBody > CurrWaterBody : AllWaterBodies)
	{
		TObjectPtr<UWaterBodyComponent> CurrWaterBodyComponent = CurrWaterBody->GetWaterBodyComponent();

		if (CurrWaterBodyComponent != nullptr)
		{
			// accumulate bounds
			FBoxSphereBounds WorldBounds;
			CurrWaterBody->GetActorBounds(true, WorldBounds.Origin, WorldBounds.BoxExtent);				

			CombinedWorldBoundsBuilder += WorldBounds;
		}
	}
	FBoxSphereBounds CombinedBounds(CombinedWorldBoundsBuilder);

	if (CombinedBounds.BoxExtent.Length() < SMALL_NUMBER)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - river bodies have zero bounds");
		return;
	}
	
	SystemPos = CombinedBounds.Origin - FVector(0, 0, CombinedBounds.BoxExtent.Z);
	WorldGridSize = 2.0f * FVector2D(CombinedBounds.BoxExtent.X, CombinedBounds.BoxExtent.Y);
	
	if (WorldGridSize.X < SMALL_NUMBER || WorldGridSize.Y < SMALL_NUMBER)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - Simulation grid has (0,0) size.");
		return;
	}

	RiverSimSystem = NewObject<UNiagaraComponent>(this, NAME_None, RF_Public);
	RiverSimSystem->bUseAttachParentBound = false;
	RiverSimSystem->SetWorldLocation(SystemPos);

	bool bReadBakedSim = RenderState == EShallowWaterRenderState::BakedSim || RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim || RenderState == EShallowWaterRenderState::WaterComponent;

	if (GetWorld() && GetWorld()->bIsWorldInitialized)
	{
		if (!RiverSimSystem->IsRegistered())
		{
			RiverSimSystem->RegisterComponentWithWorld(GetWorld());
		}

		RiverSimSystem->SetVisibleFlag(true);
		RiverSimSystem->SetAsset(NiagaraRiverSimulation);
							
		// convert to raw ptr array for function library
		if (!bReadBakedSim && bUseCapture)
		{
			// landscape captures
			TArray<AActor*> LandscapeBottomContourActorsRawPtr;
			LandscapeBottomContourActorsRawPtr.Add(nullptr);
			for (TSoftObjectPtr<AActor> CurrLandscapeActor : BottomContourLandscapeActors)
			{
				// only accept Landscapes and LandscapeStreamingProxies
				if (!Cast<ALandscape>(CurrLandscapeActor.Get()) && !Cast<ALandscapeStreamingProxy>(CurrLandscapeActor.Get()))
				{
					UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - Landscape bottom contour actors can only be ALandscape actors or ALandscapeStreamingProxy actors");
					continue;
				}

				LandscapeBottomContourActorsRawPtr.Add(CurrLandscapeActor.Get());
			}
			FBoxSphereBounds LandscapeBottomContourBounds = InitializeCaptureDI("User.LandscapeBottomCapture", LandscapeBottomContourActorsRawPtr);
		
			// undilated captures
			TArray<AActor*> BottomContourActorsRawPtr;
			BottomContourActorsRawPtr.Add(nullptr);
			AddActorsToRawArray(BottomContourActors, BottomContourActorsRawPtr);			
			AddTaggedActorsToArray(BottomContourTags, BottomContourActorsRawPtr);
			FBoxSphereBounds CombinedBottomContourBounds = InitializeCaptureDI("User.BottomCapture", BottomContourActorsRawPtr);
			FBoxSphereBounds CombinedBottomContourBoundsUnder = InitializeCaptureDI("User.BottomCaptureUnder", BottomContourActorsRawPtr);

			// Dilated capture
			TArray<AActor*> DilatedBottomContourActorsRawPtr;
			DilatedBottomContourActorsRawPtr.Add(nullptr);
			AddActorsToRawArray(DilatedBottomContourActors, DilatedBottomContourActorsRawPtr);
			AddTaggedActorsToArray(DilatedBottomContourTags, DilatedBottomContourActorsRawPtr);

			FBoxSphereBounds DilatedCombinedBottomContourBounds = InitializeCaptureDI("User.DilatedBottomCapture", DilatedBottomContourActorsRawPtr);
			FBoxSphereBounds DilatedCombinedBottomContourBoundsUnder = InitializeCaptureDI("User.DilatedBottomCaptureUnder", DilatedBottomContourActorsRawPtr);

			// reinitialize and set variables on the system
			RiverSimSystem->ReinitializeSystem();

			RiverSimSystem->SetVariableFloat(FName("LandscapeCaptureOffset"), LandscapeBottomContourBounds.Origin.Z + LandscapeBottomContourBounds.BoxExtent.Z + BottomContourCaptureOffset);			

			RiverSimSystem->SetVariableFloat(FName("CaptureOffset"), CombinedBottomContourBounds.Origin.Z + CombinedBottomContourBounds.BoxExtent.Z + BottomContourCaptureOffset);
			RiverSimSystem->SetVariableFloat(FName("DilatedCaptureOffset"), DilatedCombinedBottomContourBounds.Origin.Z + DilatedCombinedBottomContourBounds.BoxExtent.Z + BottomContourCaptureOffset);
			
			RiverSimSystem->SetVariableFloat(FName("CaptureOffsetUnder"), CombinedBottomContourBounds.Origin.Z - CombinedBottomContourBounds.BoxExtent.Z - BottomContourCaptureOffset);
			RiverSimSystem->SetVariableFloat(FName("DilatedCaptureOffsetUnder"), DilatedCombinedBottomContourBounds.Origin.Z - DilatedCombinedBottomContourBounds.BoxExtent.Z - BottomContourCaptureOffset);
		}
		else
		{
			RiverSimSystem->ReinitializeSystem();
		}
	}
	else
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - World not initialized");
		return;
	}
	

	if (RiverSimSystem == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - Cannot spawn river system");
		return;
	}

	// look for the water info texture
	for (TSoftObjectPtr<AWaterBody > CurrWaterBody : AllWaterBodies)
	{
		if (AWaterZone* WaterZone = CurrWaterBody->GetWaterBodyComponent()->GetWaterZone())
		{			
			const TObjectPtr<UTextureRenderTarget2DArray> NewWaterInfoTexture = WaterZone->WaterInfoTextureArray;
			if (NewWaterInfoTexture == nullptr)
			{
				WaterZone->GetOnWaterInfoTextureArrayCreated().RemoveDynamic(this, &UShallowWaterRiverComponent::OnWaterInfoTextureArrayCreated);
				WaterZone->GetOnWaterInfoTextureArrayCreated().AddDynamic(this, &UShallowWaterRiverComponent::OnWaterInfoTextureArrayCreated);
			}
			else
			{
				OnWaterInfoTextureArrayCreated(NewWaterInfoTexture);
			}			

			const int32 PlayerIndex = 0;
			FVector ZoneLocation;
			WaterZone->GetDynamicWaterInfoCenter(PlayerIndex, ZoneLocation);
			const FVector2D ZoneExtent = FVector2D(WaterZone->GetDynamicWaterInfoExtent());
			const FVector2D WaterHeightExtents = FVector2D(WaterZone->GetWaterHeightExtents());
			const float GroundZMin = WaterZone->GetGroundZMin();

			RiverSimSystem->SetVariableVec2(FName("WaterZoneLocation"), FVector2D(ZoneLocation));
			RiverSimSystem->SetVariableVec2(FName("WaterZoneExtent"), ZoneExtent);
			RiverSimSystem->SetVariableInt(FName("WaterZoneIdx"), WaterZone->GetWaterZoneIndex());
			
			break;
		}
	}

	RiverSimSystem->Activate();
	
	RiverSimSystem->SetVariableVec2(FName("WorldGridSize"), WorldGridSize);
	RiverSimSystem->SetVariableInt(FName("ResolutionMaxAxis"), ResolutionMaxAxis);

	// #todo(dmp): would be better to initialize the user var inside of niagara rather than recomputing res here
	double CellSize = FMath::Max(WorldGridSize.X, WorldGridSize.Y) / ResolutionMaxAxis;

	SimRes = FVector2D(ResolutionMaxAxis, FMath::FloorToInt(ResolutionMaxAxis * WorldGridSize.Y / WorldGridSize.X));
	
	if (WorldGridSize.Y > WorldGridSize.X)
	{
		SimRes = FVector2D(FMath::FloorToInt(ResolutionMaxAxis * WorldGridSize.X / WorldGridSize.Y), ResolutionMaxAxis);
	}

	if (WorldGridSize.X > WorldGridSize.Y && FMath::Abs(CellSize * SimRes.Y - WorldGridSize.Y) > SMALL_NUMBER)
	{
		SimRes.Y++;
	}

	if (WorldGridSize.X < WorldGridSize.Y && FMath::Abs(CellSize * SimRes.X - WorldGridSize.X) > SMALL_NUMBER)
	{
		SimRes.X++;
	}

	RiverSimSystem->SetVariableVec2(FName("SimRes"), SimRes);

	// pad out source's box height a so it intersects the sim plane.  This value doesn't matter much so we hardcode it
	float Overshoot = 1000.f;
	float FinalSourceHeight = 2. * CombinedBounds.BoxExtent.Z + Overshoot;

	// Get sources	
	TArray<FVector> SourcePosArray;
	TArray<FVector3f> FullSourceSizeArray;
	TArray<float> FullSourceAngleArray;
	for (TSoftObjectPtr<AWaterBody> CurrWaterBody : SourceRiverWaterBodies)
	{
		FVector CurrSourcePos;
		float CurrSourceWidth;
		float CurrSourceDepth;
		FVector CurrSourceDir;
		if (!QueryWaterAtSplinePoint(CurrWaterBody, 0, CurrSourcePos, CurrSourceDir, CurrSourceWidth, CurrSourceDepth))
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - water source query failed");
			continue;
		}		
		
		FVector FullSourcePos = CurrSourcePos - FVector(0, 0, .5 * FinalSourceHeight) + FVector(CurrSourceDir.X, CurrSourceDir.Y, 0) * .5 * SourceSize;
		FVector FullSourceSize = FVector(CurrSourceWidth, SourceSize, FinalSourceHeight); 
		
		CurrSourceDir = FVector(CurrSourceDir.X, CurrSourceDir.Y, 0);
		CurrSourceDir.Normalize();
		
		FVector BaseVector = {0,1,0};	
		double FullSourceAngle =  FMath::Acos(FVector::DotProduct(BaseVector, CurrSourceDir));
		
		FVector AxisToUse = FVector::CrossProduct(BaseVector, CurrSourceDir);
		AxisToUse.Normalize();

#if ENABLE_DRAW_DEBUG
		if (bShallowWaterRiverDebugVisualize)
		{
			FQuat TmpQ = FQuat::MakeFromRotationVector(AxisToUse * FullSourceAngle);		
			DrawDebugBox(GetWorld() , (FVector) FullSourcePos, .5 * FullSourceSize, TmpQ, FColor::Green, true);		
		}
#endif

		// flip axis so we don't need to store the vector itself
		if (AxisToUse.Z < 0)
		{
			FullSourceAngle *= -1;
		}


		SourcePosArray.Add(FullSourcePos);
		FullSourceSizeArray.Add(FVector3f(FullSourceSize));
		FullSourceAngleArray.Add(FullSourceAngle);				
	}
	UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayPosition(RiverSimSystem, "User.SourcePosArray",SourcePosArray);
	UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector(RiverSimSystem, "User.SourceSizeArray", FullSourceSizeArray);
	UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayFloat(RiverSimSystem, "User.SourceAngleArray", FullSourceAngleArray);	

	// get sinks		
	TArray<FVector> SinkPosArray;
	TArray<FVector3f> FullSinkSizeArray;
	TArray<float> FullSinkAngleArray;

	for (TSoftObjectPtr<AWaterBody> CurrWaterBody : SinkRiverWaterBodies)
	{
		FVector SinkPos(0, 0, 0);
		float SinkWidth = 1;
		float SinkDepth = 1;
		FVector SinkDir(1, 0, 0);
		if (!QueryWaterAtSplinePoint(CurrWaterBody, -1, SinkPos, SinkDir, SinkWidth, SinkDepth))
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - water sink query failed");
			continue;
		}

		// height of the sink box doesn't matter
		float SinkBoxHeight = 100000;
		FVector FullSinkSize = FVector(SinkWidth, SourceSize, SinkBoxHeight);
		
		SinkDir = FVector(SinkDir.X, SinkDir.Y, 0);
		SinkDir.Normalize();
		
		FVector BaseVector = {0,1,0};	
		double FullSinkAngle =  FMath::Acos(FVector::DotProduct(BaseVector, SinkDir));
		
		FVector AxisToUse = FVector::CrossProduct(BaseVector, SinkDir);
		AxisToUse.Normalize();

#if ENABLE_DRAW_DEBUG
		if (bShallowWaterRiverDebugVisualize)
		{
			FQuat TmpQ = FQuat::MakeFromRotationVector(AxisToUse * FullSinkAngle);		
			DrawDebugBox(GetWorld() , (FVector) SinkPos, .5 * FullSinkSize, TmpQ, FColor::Red, true);		
		}
#endif

		// flip axis so we don't need to store the vector itself
		if (AxisToUse.Z < 0)
		{
			FullSinkAngle *= -1;
		}
		
		SinkPosArray.Add(SinkPos);
		FullSinkSizeArray.Add(FVector3f(FullSinkSize));
		FullSinkAngleArray.Add(FullSinkAngle);				
	}
	UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayPosition(RiverSimSystem, "User.SinkPosArray",SinkPosArray);
	UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayVector(RiverSimSystem, "User.SinkSizeArray", FullSinkSizeArray);
	UNiagaraDataInterfaceArrayFunctionLibrary::SetNiagaraArrayFloat(RiverSimSystem, "User.SinkAngleArray", FullSinkAngleArray);		

	RiverSimSystem->SetVariableFloat(FName("SimSpeed"), SimSpeed);
	RiverSimSystem->SetVariableInt(FName("NumSteps"), NumSteps);

	RiverSimSystem->SetVariableBool(FName("MatchSpline"), bMatchSpline);
	RiverSimSystem->SetVariableFloat(FName("RemoveOutsideSplineAmount"), RemoveOutsideSplineAmount);
	RiverSimSystem->SetVariableFloat(FName("SplineHeightMatchingAmount"), MatchSplineHeightAmount);
	
	BakedWaterSurfaceRT = NewObject<UTextureRenderTarget2D>(this, NAME_None, RF_Transient);
	BakedWaterSurfaceRT->InitAutoFormat(1, 1);
	RiverSimSystem->SetVariableTextureRenderTarget(FName("SimGridRT"), BakedWaterSurfaceRT);
	
	BakedFoamRT = NewObject<UTextureRenderTarget2D>(this, NAME_None, RF_Transient);
	BakedFoamRT->InitAutoFormat(1, 1);
	RiverSimSystem->SetVariableTextureRenderTarget(FName("FoamRT"), BakedFoamRT);
	
	BakedWaterSurfaceNormalRT = NewObject<UTextureRenderTarget2D>(this, NAME_None, RF_Transient);
	BakedWaterSurfaceNormalRT->InitAutoFormat(1, 1);
	RiverSimSystem->SetVariableTextureRenderTarget(FName("NormalRT"), BakedWaterSurfaceNormalRT);

	RiverSimSystem->SetVariableBool(FName("ReadCachedSim"), bReadBakedSim);

	RiverSimSystem->SetVariableFloat(FName("BottomContourCollisionDilation"), BottomContourCollisionDilation);
	
	RiverSimSystem->SetVariableInt(FName("ExtrapolationHalfWidth"), SmoothingWidth);
	RiverSimSystem->SetVariableFloat(FName("SmoothingHeightCutoff"), SmoothingCutoff);

	if (BakedWaterSurfaceTexture != nullptr && BakedFoamTexture != nullptr && BakedWaterSurfaceNormalTexture != nullptr)
	{
		RiverSimSystem->SetVariableTexture(FName("BakedSimTexture"), BakedWaterSurfaceTexture);
		RiverSimSystem->SetVariableTexture(FName("BakedFoamTexture"), BakedFoamTexture);
		RiverSimSystem->SetVariableTexture(FName("BakedWaterSurfaceNormalTexture"), BakedWaterSurfaceNormalTexture);
	}

	TObjectPtr<UTextureRenderTarget2D>  OceanPatchNormalRT = GetSharedFFTOceanPatchNormalRTFromSubsystem(GetWorld());

	if (OceanPatchNormalRT == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - ocean patch normal RT is not initialized");
		return;
	}

	NormalDetailRT = OceanPatchNormalRT;
	RiverSimSystem->SetVariableTextureRenderTarget(FName("NormalDetailRT"), NormalDetailRT);

#if WITH_EDITOR
	// run the live sim and make sure that we enable/disable cpu throttling	
	if (!bReadBakedSim)
	{		
		GEditor->ShouldDisableCPUThrottlingDelegates.Add(UEditorEngine::FShouldDisableCPUThrottling::CreateUObject(this, &UShallowWaterRiverComponent::ShouldDisableCPUThrottling));
		ShouldDisableCPUThrottlingDelegateHandle = GEditor->ShouldDisableCPUThrottlingDelegates.Last().GetHandle();
	}
	else
	{
		GEditor->ShouldDisableCPUThrottlingDelegates.RemoveAll([this](const UEditorEngine::FShouldDisableCPUThrottling& Delegate)
		{
			return Delegate.GetHandle() == ShouldDisableCPUThrottlingDelegateHandle;
		});
	}
#endif

	bIsInitialized = true;
}

void UShallowWaterRiverComponent::AddActorsToRawArray(const TArray<TSoftObjectPtr<AActor>> &ActorsArray, TArray<AActor*>& BottomContourActorsRawPtr)
{
	for (TSoftObjectPtr<AActor> CurrActor : ActorsArray)
	{
		AActor* CurrActorRawPtr = CurrActor.Get();

		// if we have a level instance, break it up and add each actor
		if (ALevelInstance* LevelInstancePtr = Cast<ALevelInstance>(CurrActorRawPtr))
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();

			LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstancePtr, [&](AActor* SubActor)
				{
					BottomContourActorsRawPtr.Add(SubActor);
					return true;
				});
		}
		else if (bRecursivelyAddAttachedActors)
		{
			TArray<AActor*> Meshes;
			CurrActorRawPtr->GetAttachedActors(Meshes);
			for (AActor* AttachedMesh : Meshes)
			{
				BottomContourActorsRawPtr.Add(AttachedMesh);
			}
		}
		else
		{
			BottomContourActorsRawPtr.Add(CurrActorRawPtr);
		}
	}
}

void UShallowWaterRiverComponent::AddTaggedActorsToArray(TArray<FName> &TagsToUse, TArray<AActor*>& BottomContourActorsRawPtr)
{
	// if we have a tag set
	// do an overlap test
	//  filter by tag and add to the bottomcontour actors list
	//  if a level instance is tagged, loop over the contained actors

	if (!TagsToUse.IsEmpty())
	{
		FCollisionQueryParams Params(SCENE_QUERY_STAT(ShallowWaterRiverActorQuery), false);

		TArray<FOverlapResult> Overlaps;
		GetWorld()->OverlapMultiByChannel(Overlaps, SystemPos, FQuat::Identity, ECollisionChannel::ECC_WorldStatic,
			FCollisionShape::MakeBox(0.5f * FVector(WorldGridSize.X, WorldGridSize.Y, 100000)), Params);

		for (const FOverlapResult& OverlapResult : Overlaps)
		{
			if (UPrimitiveComponent* PrimitiveComponent = OverlapResult.GetComponent())
			{
				if (AActor* ComponentActor = PrimitiveComponent->GetOwner())
				{
					if (TagsToUse.ContainsByPredicate([&](const FName& Tag) { return Tag == NAME_None || ComponentActor->Tags.Contains(Tag); }))
					{
						// if we have a level instance, break it up and add each actor
						if (ALevelInstance* LevelInstancePtr = Cast<ALevelInstance>(ComponentActor))
						{
							const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();

							LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstancePtr, [&](AActor* SubActor)
								{
									BottomContourActorsRawPtr.Add(SubActor);
									return true;
								});
						}
						else
						{
							BottomContourActorsRawPtr.Add(ComponentActor);
						}
					}
				}
			}
		}
	}
}

void UShallowWaterRiverComponent::Bake()
{
	EObjectFlags TextureObjectFlags = EObjectFlags::RF_Public;

	if (!RiverSimSystem || !BakedWaterSurfaceRT || !BakedFoamRT || !BakedWaterSurfaceNormalRT)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Bake() - No simulation to bake");
		return;
	}

	if (RenderState != EShallowWaterRenderState::LiveSim)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Bake() - Must be in live sim mode to bake");
		return;
	}

	if (SourceRiverWaterBodies.Num() != 0)
	{
		for (TSoftObjectPtr<AWaterBody > CurrWaterBody : SourceRiverWaterBodies)
		{
			if (CurrWaterBody == nullptr)
			{			
				UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Bake() - Cannot bake with a null water body.  Please make sure all water bodies are loaded and that all entries on the actor are valid");
				return;
			}
		}
	}	

	SimRes = FVector2D(BakedWaterSurfaceRT->SizeX, BakedWaterSurfaceRT->SizeY);
	RiverSimSystem->SetVariableVec2(FName("SimRes"), SimRes);

	if (WorldGridSize.X > WorldGridSize.Y)
	{
		SimDx = WorldGridSize.X / ResolutionMaxAxis;
	}
	else
	{
		SimDx = WorldGridSize.Y / ResolutionMaxAxis;
	}		

	BakedWaterSurfaceTexture = BakedWaterSurfaceRT->ConstructTexture2D(this, "BakedRiverTexture", TextureObjectFlags);
	
	if (bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedWaterSurfaceTexture);
	}

	RiverSimSystem->SetVariableTexture(FName("BakedSimTexture"), BakedWaterSurfaceTexture);

	// Readback to get the river texture values as an array
	TArray<FFloat16Color> TmpShallowWaterSimArrayValues;
	BakedWaterSurfaceRT->GameThread_GetRenderTargetResource()->ReadFloat16Pixels(TmpShallowWaterSimArrayValues);

	// bake foam and other data to texture
	BakedFoamTexture = BakedFoamRT->ConstructTexture2D(this, "BakedFoamTexture", TextureObjectFlags);
	
	if (bUseVirtualTextures)
	{		
		InitializeVirtualTexture(BakedFoamTexture);
	}
	
	RiverSimSystem->SetVariableTexture(FName("BakedFoamTexture"), BakedFoamTexture);
	
	// bake normal to texture
	BakedWaterSurfaceNormalTexture = BakedWaterSurfaceNormalRT->ConstructTexture2D(this, "BakedWaterSurfaceNormalTexture", TextureObjectFlags);
	
	if (bUseVirtualTextures)
	{
		InitializeVirtualTexture(BakedWaterSurfaceNormalTexture);
	}
	
	RiverSimSystem->SetVariableTexture(FName("BakedWaterSurfaceNormalTexture"), BakedWaterSurfaceNormalTexture);

	// clear references to old baked sim on water body actors
	if (BakedSim != nullptr)
	{ 
		for (TSoftObjectPtr<AWaterBody > CurrWaterBody : BakedSim->WaterBodies)
		{
			if (CurrWaterBody != nullptr)
			{
				TObjectPtr<UWaterBodyComponent> CurrWaterBodyComponent = CurrWaterBody->GetWaterBodyComponent();

				if (CurrWaterBodyComponent != nullptr)
				{
					CurrWaterBodyComponent->SetBakedShallowWaterSimulation(nullptr);
					CurrWaterBodyComponent->PostEditChange();
				}
			}
		}
	}

	BakedSim = NewObject<UBakedShallowWaterSimulationComponent>(this, NAME_None, RF_Public);
	
	TObjectPtr<UShallowWaterSimulationDataSparse> BakedSimulationData = NewObject<UShallowWaterSimulationDataSparse>(BakedSim, NAME_None, RF_Public);	
	BakedSim->BakedSimulationData = BakedSimulationData;

	BakedSimulationData->Build(TmpShallowWaterSimArrayValues, FIntVector2(BakedWaterSurfaceRT->SizeX, BakedWaterSurfaceRT->SizeY));
	
	if (BakedSimulationData->GetTotalNumCells() > 0)
	{
		UE_LOGF(LogShallowWater, Log, "Baked Shallow Water Simulation: Sparseness: %f", ((float) BakedSimulationData->GetNumAllocatedCells()) / BakedSimulationData->GetTotalNumCells());
	}
	UE_LOGF(LogShallowWater, Log, "Baked Shallow Water Simulation: CPU Memory Usage: %d", BakedSimulationData->GetMemoryInBytes());

	BakedSimulationData->Position = SystemPos;
	BakedSimulationData->Size = WorldGridSize;
	BakedSimulationData->BakedTexture = BakedWaterSurfaceTexture;

	BakedSim->WaterBodies = AllWaterBodies;

	// Compute the maximum water height for each convex in each water body simulated by this river.
	// We use this to modify the collision geometry so it fully encompasses the baked water sim.

	// Gather all convex elements once to avoid redundant lookups per cell
	struct FConvexInfo
	{
		FKConvexElem* Convex;
		FTransform Transform;
		FBox AABBox;
		USplineMeshComponent* SplineComponent; // Needed for PostEditChange after vertex modification
	};
	TArray<FConvexInfo> AllConvexes;

	for (TSoftObjectPtr<AWaterBody> CurrWaterBody : AllWaterBodies)
	{
		if (TObjectPtr<UWaterBodyComponent> CurrWaterBodyComponent = CurrWaterBody->GetWaterBodyComponent())
		{		
			TArray<UPrimitiveComponent*> CollisionComponents = CurrWaterBodyComponent->GetCollisionComponents();

			for (UPrimitiveComponent* CurrCollisionComponent : CollisionComponents)
			{
				if (USplineMeshComponent* CurrSplineComponent = Cast<USplineMeshComponent>(CurrCollisionComponent))
				{
					if (TObjectPtr<UBodySetup> CurrBodySetup = CurrSplineComponent->BodySetup)
					{
						const FTransform CurrMeshTransform = CurrCollisionComponent->GetComponentTransform();
						for (FKConvexElem& ConvexElem : CurrBodySetup->AggGeom.ConvexElems)
						{
							const FBox CurrBox = ConvexElem.CalcAABB(CurrMeshTransform, FVector(1, 1, 1));
							AllConvexes.Add({ &ConvexElem, CurrMeshTransform, CurrBox, CurrSplineComponent });
						}
					}
				}
			}
		}
	}

	// Iterate over allocated blocks only (sparse storage is always used - see line 929)
	
	// Map each convex to the maximum water height found within its bounds
	TMap<FKConvexElem*, float> ConvexToMaxHeight;

	for (const auto& BlockPair : BakedSimulationData->GridIndexToBlockIndex)
	{
		const FIntVector2 BlockIndex = BlockPair.Key;

		// Compute cell range for this block
		const int32 StartX = BlockIndex.X * UShallowWaterSimulationDataSparse::BlockSize;
		const int32 StartY = BlockIndex.Y * UShallowWaterSimulationDataSparse::BlockSize;
		const int32 EndX = FMath::Min(StartX + UShallowWaterSimulationDataSparse::BlockSize, BakedWaterSurfaceRT->SizeX);
		const int32 EndY = FMath::Min(StartY + UShallowWaterSimulationDataSparse::BlockSize, BakedWaterSurfaceRT->SizeY);

		// Process only cells within this allocated block
		for (int32 y = StartY; y < EndY; ++y) {
		for (int32 x = StartX; x < EndX; ++x) {
			FVector WorldPos = BakedSimulationData->IndexToWorld(FIntVector2(x, y));

			FVector Vel;
			float Height, Depth;
			BakedSimulationData->QueryShallowWaterSimulationAtIndex(FIntVector2(x, y), Vel, Height, Depth);
			WorldPos.Z = Height;

			// Depth check still needed since sparse blocks can have some zero-depth cells
			if (Depth > 1e-5)
			{
				// Test against pre-gathered convex elements
				for (const FConvexInfo& ConvexInfo : AllConvexes)
				{
					// See if the current point is inside the convex projected to the xy plane
					if (ConvexInfo.AABBox.IsInsideXY(FBox(WorldPos, WorldPos)))
					{
						float* TmpMaxHeight = ConvexToMaxHeight.Find(ConvexInfo.Convex);
						if (TmpMaxHeight == nullptr)
						{
							ConvexToMaxHeight.Emplace(ConvexInfo.Convex, WorldPos.Z);
						}
						else
						{
							*TmpMaxHeight = FMath::Max(*TmpMaxHeight, WorldPos.Z);
						}
					}
				}
			}
		}}
	}

	// Apply the maximum water heights to convex collision geometry
	// Use AllConvexes list to avoid redundant component traversal
	TSet<USplineMeshComponent*> ModifiedSplineComponents;
	for (const FConvexInfo& ConvexInfo : AllConvexes)
	{
		if (const float* WorldMaxZForConvex = ConvexToMaxHeight.Find(ConvexInfo.Convex))
		{
			TArray<FVector>& VertexData = ConvexInfo.Convex->VertexData;

			// For each vertex in the convex hull, set the Z to the maximum baked water sim Z height
			int32 Idx = 0;
			for (FVector& Vertex : VertexData)
			{
				// Only top vertices are 4,5,6,7
				if (Idx >= 4)
				{
					FVector VWorld = ConvexInfo.Transform.TransformPosition(Vertex);
					VWorld.Z = FMath::Max(VWorld.Z, *WorldMaxZForConvex);

					const FVector VLocal = ConvexInfo.Transform.InverseTransformPosition(VWorld);
					Vertex.X = VLocal.X;
					Vertex.Y = VLocal.Y;
					Vertex.Z = VLocal.Z;
				}

				#if ENABLE_DRAW_DEBUG
				if (bShallowWaterRiverDebugVisualize)
				{
					FVector VWorld = ConvexInfo.Transform.TransformPosition(Vertex);

					switch (Idx)
					{
						case 0:
						DrawDebugSphere(GetWorld(), VWorld, 10., 2, FColor::Red, true);
						break;
						case 1:
						DrawDebugSphere(GetWorld(), VWorld, 10., 3, FColor::Green, true);
						break;
						case 2:
						DrawDebugSphere(GetWorld(), VWorld, 10., 4, FColor::Blue, true);
						break;
						case 3:
						DrawDebugSphere(GetWorld(), VWorld, 10., 5, FColor::Black, true);
						break;
						case 4:
						DrawDebugSphere(GetWorld(), VWorld, 10., 6, FColor::White, true);
						break;
						case 5:
						DrawDebugSphere(GetWorld(), VWorld, 10., 7, FColor::Magenta, true);
						break;
						case 6:
						DrawDebugSphere(GetWorld(), VWorld, 10., 8, FColor::Orange, true);
						break;
						case 7:
						DrawDebugSphere(GetWorld(), VWorld, 10., 9, FColor::Purple, true);
						break;
					}
				}
				#endif

				Idx++;
			}

			// Track modified components for PostEditChange
			ModifiedSplineComponents.Add(ConvexInfo.SplineComponent);
		}
	}

	// Notify modified spline components
	for (USplineMeshComponent* SplineComponent : ModifiedSplineComponents)
	{
		SplineComponent->PostEditChange();
	}

	// Set the sim texture on each water body that is in the simulated river
	for (TSoftObjectPtr<AWaterBody> CurrWaterBody : AllWaterBodies)
	{
		if (TObjectPtr<UWaterBodyComponent> CurrWaterBodyComponent = CurrWaterBody->GetWaterBodyComponent())		
		{
			CurrWaterBodyComponent->SetBakedShallowWaterSimulation(BakedSim);
			CurrWaterBodyComponent->PostEditChange();
		}
	}
}

void UShallowWaterRiverComponent::InitializeVirtualTexture(TObjectPtr<UTexture2D> InTexture)
{	
	InTexture->Modify();
	InTexture->MipGenSettings = TextureMipGenSettings::TMGS_SimpleAverage;
	InTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::StretchToPowerOfTwo;
	InTexture->VirtualTextureStreaming = true;

	InTexture->UpdateResource();	
	InTexture->WaitForStreaming(UStreamableRenderAsset::ETickStreamingFlags::SendCompletionEvents);
	InTexture->BlockOnAnyAsyncBuild();
	InTexture->PostEditChange();
}

bool UShallowWaterRiverComponent::QueryWaterAtSplinePoint(TSoftObjectPtr<AWaterBody> WaterBody, int SplinePoint, FVector& OutPos, FVector& OutTangent, float& OutWidth, float& OutDepth)
{	
	if (WaterBody != nullptr)
	{
		TObjectPtr<UWaterBodyComponent> CurrWaterBodyComponent = WaterBody->GetWaterBodyComponent();

		UWaterSplineComponent* CurrSpline = WaterBody->GetWaterSpline();
		
		if (CurrSpline != nullptr)
		{
			// -1 means last spline point
			if (SplinePoint == -1)
			{
				SplinePoint = CurrSpline->GetNumberOfSplinePoints() - 1;
			}

			UWaterSplineMetadata* Metadata = WaterBody->GetWaterSplineMetadata();

			if (Metadata != nullptr)
			{
				OutPos = CurrSpline->GetLocationAtSplineInputKey(SplinePoint, ESplineCoordinateSpace::Local);
				OutPos = CurrSpline->GetComponentTransform().TransformPosition(OutPos);

				OutWidth = Metadata->RiverWidth.Points[SplinePoint].OutVal;
				OutDepth = Metadata->Depth.Points[SplinePoint].OutVal;

				OutTangent = CurrSpline->GetLeaveTangentAtSplinePoint(SplinePoint, ESplineCoordinateSpace::Local);

				OutTangent = CurrSpline->GetComponentTransform().TransformVector(OutTangent);
				OutTangent.Normalize();
			}
			else
			{
				UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::QueryWaterAtSplinePoint() - Water spline metadata is null");
				return false;
			}
		}
		else
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::QueryWaterAtSplinePoint() - Water spline component is null");
			return false;
		}
	}
	else
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::QueryWaterAtSplinePoint() - Water actor is null");
		return false;
	}

	return true;
}

void UShallowWaterRiverComponent::OnWaterInfoTextureArrayCreated(const UTextureRenderTarget2DArray* InWaterInfoTexture)
{	
	if (InWaterInfoTexture == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::OnWaterInfoTextureCreated was called with NULL WaterInfoTexture");
		return;
	}
	
	WaterInfoTexture = InWaterInfoTexture;
	if (RiverSimSystem)
	{
		UTexture* WITTextureArray = Cast<UTexture>(const_cast<UTextureRenderTarget2DArray*>(WaterInfoTexture.Get()));
		if (WITTextureArray == nullptr)
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::OnWaterInfoTextureCreated was called with Water Info Texture that isn't valid");
			return;
		}

		RiverSimSystem->SetVariableTexture(FName("WaterInfoTexture"), WITTextureArray);
	}
	else
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::OnWaterInfoTextureCreated was called with NULL ShallowWaterNiagaraSimulation");
		return;
	}
}
#endif

void UShallowWaterRiverComponent::SetPaused(bool Pause)
{
	if (RiverSimSystem)
	{
		RiverSimSystem->SetPaused(Pause);
	}

	UFFTOceanPatchSubsystem *OceanPatchSubsystem = GetWorld()->GetSubsystem<UFFTOceanPatchSubsystem>();
	if (OceanPatchSubsystem != nullptr)
	{
		TObjectPtr<UNiagaraComponent> OceanSystem = OceanPatchSubsystem->GetOceanSystem();
		if (OceanSystem)
		{
			OceanSystem->SetPaused(Pause);
		}
	}
}

void UShallowWaterRiverComponent::UpdateRenderState()
{
	TObjectPtr<UTextureRenderTarget2D>  OceanPatchNormalRT = GetSharedFFTOceanPatchNormalRTFromSubsystem(GetWorld());

	if (OceanPatchNormalRT == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::Rebuild() - ocean patch normal RT is not initialized");
		return;
	}
	
	NormalDetailRT = OceanPatchNormalRT;

	if (BakedSimMaterial == nullptr)
	{
		BakedSimMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Niagara/Materials/SW_Water_Material_River.SW_Water_Material_River"));
	}
	
	if (BakedSimUnderWaterMaterial == nullptr)
	{
		BakedSimUnderWaterMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Materials/M_UnderWater_PostProcess_Volume_SW.M_UnderWater_PostProcess_Volume_SW"));
	}

	if (BakedSimRiverToLakeTransitionMaterial == nullptr)
	{
		BakedSimRiverToLakeTransitionMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Niagara/Materials/SW_Water_Material_River_To_Lake_Transition.SW_Water_Material_River_To_Lake_Transition"));
	}

	if (BakedSimRiverToOceanTransitionMaterial == nullptr)
	{
		BakedSimRiverToOceanTransitionMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Niagara/Materials/SW_Water_Material_River_To_Ocean_Transition.SW_Water_Material_River_To_Ocean_Transition"));
	}
	
	if (SplineRiverMaterial == nullptr)
	{
		SplineRiverMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Niagara/Materials/SW_Water_Material_River_Spline.SW_Water_Material_River_Spline"));
	}

	if (SplineRiverToLakeTransitionMaterial == nullptr)
	{
		SplineRiverToLakeTransitionMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Niagara/Materials/SW_Water_Material_River_To_Lake_Transition_Spline.SW_Water_Material_River_To_Lake_Transition_Spline"));
	}

	if (SplineRiverToOceanTransitionMaterial == nullptr)
	{
		SplineRiverToOceanTransitionMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Niagara/Materials/SW_Water_Material_River_To_Ocean_Transition_Spline.SW_Water_Material_River_To_Ocean_Transition_Spline"));
	}

	if (SplineRiverUnderWaterMaterial == nullptr)
	{
		SplineRiverUnderWaterMaterial = LoadObject<UMaterialInstance>(nullptr, TEXT("/WaterAdvanced/Materials/M_UnderWater_PostProcess_Volume_Spline.M_UnderWater_PostProcess_Volume_Spline"));
	}

	bool bReadBakedSim = RenderState == EShallowWaterRenderState::BakedSim || RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim || RenderState == EShallowWaterRenderState::WaterComponent;
	bool RenderWaterBody = RenderState == EShallowWaterRenderState::WaterComponent || RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim;
	bool RenderSecondary = 
		RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim || 
		RenderState == EShallowWaterRenderState::BakedSim || RenderState == EShallowWaterRenderState::LiveSim;

	if (RiverSimSystem != nullptr)
	{		
		RiverSimSystem->SetVariableVec2(FName("SimRes"), SimRes);

		RiverSimSystem->SetVariableBool(FName("RenderWaterSurface"), !RenderWaterBody);
		RiverSimSystem->SetVariableBool(FName("RenderSecondary"), RenderSecondary);
		RiverSimSystem->SetVariableBool(FName("DebugRenderBottomContour"), RenderState == EShallowWaterRenderState::DebugRenderBottomContour);
		RiverSimSystem->SetVariableBool(FName("DebugRenderFoam"), RenderState == EShallowWaterRenderState::DebugRenderFoam);
		RiverSimSystem->SetVariableBool(FName("ReadCachedSim"), bReadBakedSim);
		
		RiverSimSystem->SetVariableTextureRenderTarget("OceanNormalRT", NormalDetailRT);
		RiverSimSystem->ReinitializeSystem();
	}

	if ((RenderState == EShallowWaterRenderState::BakedSim || RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim) && 
		(BakedWaterSurfaceTexture == nullptr || BakedWaterSurfaceTexture->GetSizeX() == 0 || BakedWaterSurfaceTexture->GetSizeY() == 0))
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::UpdateRenderState() - No baked sim to render");		
	}

	for (TSoftObjectPtr<AWaterBody > CurrWaterBody : AllWaterBodies)
	{
		if (!CurrWaterBody)
		{
			UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::UpdateRenderState() - Water Body Actor is null- skipping setting render state");
			continue;
		}

		TObjectPtr<UWaterBodyRiverComponent> CurrWaterBodyComponent = Cast<UWaterBodyRiverComponent>(CurrWaterBody->GetWaterBodyComponent());

		if (CurrWaterBodyComponent != nullptr)
		{
			CurrWaterBodyComponent->SetVisibility(RenderWaterBody);

			if (RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim)
			{				
				CurrWaterBodyComponent->SetWaterMaterial(BakedSimMaterial);
				UMaterialInstanceDynamic* WaterMID = CurrWaterBodyComponent->GetWaterMaterialInstance();			
				SetWaterMIDParameters(WaterMID);

				CurrWaterBodyComponent->SetUnderwaterPostProcessMaterial(BakedSimUnderWaterMaterial);
				UMaterialInstanceDynamic* UnderWaterMID = CurrWaterBodyComponent->GetUnderwaterPostProcessMaterialInstance();
				SetWaterMIDParameters(UnderWaterMID);

				CurrWaterBodyComponent->SetLakeTransitionMaterial(BakedSimRiverToLakeTransitionMaterial);
				UMaterialInstanceDynamic* WaterLakeTransitionMID = CurrWaterBodyComponent->GetRiverToLakeTransitionMaterialInstance();
				SetWaterMIDParameters(WaterLakeTransitionMID);

				CurrWaterBodyComponent->SetOceanTransitionMaterial(BakedSimRiverToOceanTransitionMaterial);
				UMaterialInstanceDynamic* WaterOceanTransitionMID = CurrWaterBodyComponent->GetRiverToOceanTransitionMaterialInstance();
				SetWaterMIDParameters(WaterOceanTransitionMID);
												
				UMaterialInstanceDynamic* WaterInfoMID = CurrWaterBodyComponent->GetWaterInfoMaterialInstance();
				if (WaterInfoMID)
				{
					WaterInfoMID->SetTextureParameterValue("BakedWaterSimTex", BakedWaterSurfaceTexture);
					WaterInfoMID->SetTextureParameterValue("FoamTex", BakedFoamTexture);
					WaterInfoMID->SetTextureParameterValue("BakedWaterSimNormalTex", BakedWaterSurfaceNormalTexture);
					WaterInfoMID->SetVectorParameterValue("BakedWaterSimLocation", SystemPos);
					WaterInfoMID->SetDoubleVectorParameterValue("BakedWaterSimLocationDouble", SystemPos);
					WaterInfoMID->SetVectorParameterValue("BakedWaterSimSize", FVector(WorldGridSize.X, WorldGridSize.Y, 1));
				}
				else
				{
					UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::UpdateRenderState() - Water Component Water Info MID is null");
					return;
				}
			}
			else if (RenderState == EShallowWaterRenderState::WaterComponent)
			{
				CurrWaterBodyComponent->SetWaterMaterial(SplineRiverMaterial);
				CurrWaterBodyComponent->SetLakeTransitionMaterial(SplineRiverToLakeTransitionMaterial);
				CurrWaterBodyComponent->SetOceanTransitionMaterial(SplineRiverToOceanTransitionMaterial);
				CurrWaterBodyComponent->SetUnderwaterPostProcessMaterial(SplineRiverUnderWaterMaterial);
			}

			CurrWaterBodyComponent->SetUseBakedSimulationForQueriesAndPhysics(
				RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim || RenderState == EShallowWaterRenderState::BakedSim);

			/*
			// #todo(dmp): I'd prefer if we could set an editor time only static switch to control using baked sims in the material or not
			TArray<FMaterialParameterInfo> OutMaterialParameterInfos;
			TArray<FGuid> Guids;
			WaterMID->GetAllStaticSwitchParameterInfo(OutMaterialParameterInfos, Guids);

			for (FMaterialParameterInfo& MaterialParameterInfo : OutMaterialParameterInfos)
			{
				if (MaterialParameterInfo.Name == "UseBakedSim")
				{
					WaterMID->SetStaticSwitchParameterValueEditorOnly(MaterialParameterInfo, RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim);
				}
			}
			*/			
		}
	}

	bRenderStateTickInitialize = true;
}

void UShallowWaterRiverComponent::SetWaterMIDParameters(UMaterialInstanceDynamic* WaterMID)
{
	if (WaterMID)
	{		
		if (bUseVirtualTextures)
		{
			WaterMID->SetTextureParameterValue("BakedWaterSimTexVT", BakedWaterSurfaceTexture);
			WaterMID->SetTextureParameterValue("FoamTexVT", BakedFoamTexture);
			WaterMID->SetTextureParameterValue("BakedWaterSimNormalTexVT", BakedWaterSurfaceNormalTexture);

			UTexture2D *EmptyPtr = nullptr;
			WaterMID->SetTextureParameterValue("BakedWaterSimTex", EmptyPtr);
			WaterMID->SetTextureParameterValue("FoamTex", EmptyPtr);
			WaterMID->SetTextureParameterValue("BakedWaterSimNormalTex", EmptyPtr);
		}
		else
		{
			WaterMID->SetTextureParameterValue("BakedWaterSimTex", BakedWaterSurfaceTexture);
			WaterMID->SetTextureParameterValue("FoamTex", BakedFoamTexture);
			WaterMID->SetTextureParameterValue("BakedWaterSimNormalTex", BakedWaterSurfaceNormalTexture);

			UTexture2D *EmptyPtr = nullptr;
			WaterMID->SetTextureParameterValue("BakedWaterSimTexVT", EmptyPtr);
			WaterMID->SetTextureParameterValue("FoamTexVT", EmptyPtr);
			WaterMID->SetTextureParameterValue("BakedWaterSimNormalTexVT", EmptyPtr);
		}


		WaterMID->SetVectorParameterValue("BakedWaterSimLocation", SystemPos);
		WaterMID->SetDoubleVectorParameterValue("BakedWaterSimLocationDouble", SystemPos);
		WaterMID->SetVectorParameterValue("BakedWaterSimSize", FVector(WorldGridSize.X, WorldGridSize.Y, 1));

		WaterMID->SetTextureParameterValue("NormalDetailTex", NormalDetailRT);

		WaterMID->SetScalarParameterValue("BakedWaterSimDx", SimDx);

		WaterMID->SetScalarParameterValue("UseBakedSimHack", RenderState == EShallowWaterRenderState::WaterComponentWithBakedSim ? 1 : 0);

		WaterMID->SetVectorParameterValue("SimRes", FVector(SimRes.X, SimRes.Y, 0));
	}
	else
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterRiverComponent::UpdateRenderState() - Water Component MID is null");
		return;
	}
}

AShallowWaterRiver::AShallowWaterRiver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ShallowWaterRiverComponent = CreateDefaultSubobject<UShallowWaterRiverComponent>(TEXT("ShallowWaterRiverComponent"));
	RootComponent = ShallowWaterRiverComponent;

	PrimaryActorTick.bCanEverTick = true;
	SetHidden(false);
}

