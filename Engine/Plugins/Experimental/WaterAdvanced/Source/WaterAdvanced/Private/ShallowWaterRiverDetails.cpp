// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "ShallowWaterRiverDetails.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "GameDelegates.h"
#include "IAssetTools.h"
#include "IDetailChildrenBuilder.h"
#include "ShallowWaterRiverActor.h"
#include "Engine/World.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "ShallowWaterRiverDetails"

TSharedRef<IDetailCustomization> FShallowWaterRiverDetails::MakeInstance()
{
	return MakeShareable(new FShallowWaterRiverDetails);
}

FShallowWaterRiverDetails::~FShallowWaterRiverDetails()
{
	/*
	if (GEngine)
	{
		GEngine->OnWorldDestroyed().RemoveAll(this);
	}

	FGameDelegates::Get().GetEndPlayMapDelegate().RemoveAll(this);
	*/
}

void FShallowWaterRiverDetails::OnPiEEnd()
{
	UE_LOGF(LogTemp, Log, "onPieEnd");
	if (Component.IsValid())
	{
		if (Component->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			UE_LOGF(LogTemp, Log, "onPieEnd - has package flags");
			UWorld* TheWorld = UWorld::FindWorldInPackage(Component->GetOutermost());
			if (TheWorld)
			{
				OnWorldDestroyed(TheWorld);
			}
		}
	}
}

void FShallowWaterRiverDetails::OnWorldDestroyed(class UWorld* InWorld)
{
	// We have to clear out any temp data interfaces that were bound to the component's package when the world goes away or otherwise
	// we'll report GC leaks..
	if (Component.IsValid())
	{
		if (Component->GetWorld() == InWorld)
		{
			UE_LOGF(LogTemp, Log, "OnWorldDestroyed - matched up");
			Builder = nullptr;
		}
	}
}

void FShallowWaterRiverDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	Builder = &DetailBuilder;

	static const FName ParamUtilitiesName = TEXT("ShallowWaterRiverComponent_Utilities");
	IDetailCategoryBuilder& CustomCategory = DetailBuilder.EditCategory(ParamUtilitiesName, LOCTEXT("ParamUtilsCategoryName", "Utilities"), ECategoryPriority::Important);

	CustomCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.MaxDesiredWidth(300.f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FShallowWaterRiverDetails::OnResetSelectedSystem)
					.ToolTipText(LOCTEXT("ResetSystemButtonTooltip", "Resets the river system."))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ResetSystemButton", "Reset"))
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FShallowWaterRiverDetails::OnBakeSelectedSystem)
					.ToolTipText(LOCTEXT("BakeSystemButtonTooltip", "Bakes the river system."))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("BakeSystemButton", "Bake"))
					]
				]
			]
		];
}

FReply FShallowWaterRiverDetails::OnResetSelectedSystem()
{
	if (!Builder)
		return FReply::Handled();

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = Builder->GetSelectedObjects();

	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (AActor* Actor = Cast<AActor>(SelectedObjects[Idx].Get()))
			{
				for (UActorComponent* AC : Actor->GetComponents())
				{
					if (AC)
					{
						if (UShallowWaterRiverComponent* ShallowWaterComponent = Cast<UShallowWaterRiverComponent>(AC))
						{
							ShallowWaterComponent->RenderState = EShallowWaterRenderState::LiveSim;
							ShallowWaterComponent->Rebuild();
							ShallowWaterComponent->UpdateRenderState();	
							ShallowWaterComponent->ReregisterComponent();
						}
					}
				}
			}
			else if (UShallowWaterRiverComponent* ShallowWaterComponent = Cast<UShallowWaterRiverComponent>(SelectedObjects[Idx].Get()))
			{
				ShallowWaterComponent->RenderState = EShallowWaterRenderState::LiveSim;
				ShallowWaterComponent->UpdateRenderState();
				ShallowWaterComponent->Rebuild();
				ShallowWaterComponent->ReregisterComponent();
			}
			
		}
	}

	return FReply::Handled();
}

FReply FShallowWaterRiverDetails::OnBakeSelectedSystem()
{
	if (!Builder)
		return FReply::Handled();

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = Builder->GetSelectedObjects();

	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (AActor* Actor = Cast<AActor>(SelectedObjects[Idx].Get()))
			{
				for (UActorComponent* AC : Actor->GetComponents())
				{
					if (AC)
					{
						if (UShallowWaterRiverComponent* ShallowWaterComponent = Cast<UShallowWaterRiverComponent>(AC))
						{
							ShallowWaterComponent->Bake();
							break;
						}
					}
				}
			}
			else if (UShallowWaterRiverComponent* ShallowWaterComponent = Cast<UShallowWaterRiverComponent>(SelectedObjects[Idx].Get()))
			{
				ShallowWaterComponent->Bake();
			}

		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
#endif