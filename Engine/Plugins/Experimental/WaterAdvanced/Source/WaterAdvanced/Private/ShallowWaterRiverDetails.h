// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
//#include "ShallowWaterRiverDetails.generated.h"

class UShallowWaterRiverComponent;

class FShallowWaterRiverDetails : public IDetailCustomization
{
public:
	virtual ~FShallowWaterRiverDetails() override;

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

protected:
	void OnWorldDestroyed(class UWorld* InWorld);
	void OnPiEEnd();

	FReply OnResetSelectedSystem();
	FReply OnBakeSelectedSystem();
	
private:
	TWeakObjectPtr<UShallowWaterRiverComponent> Component;
	IDetailLayoutBuilder* Builder = nullptr;

};
#endif