// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShallowWaterSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShallowWaterSettings)

UShallowWaterSettings::UShallowWaterSettings()
{
	DefaultShallowWaterNiagaraSimulation = TSoftObjectPtr<UNiagaraSystem>(FSoftObjectPath(TEXT("/WaterAdvanced/Niagara/Systems/Grid2D_SW_WaterBody.Grid2D_SW_WaterBody")));
	DefaultShallowWaterCollisionNDC = TSoftObjectPtr<UNiagaraDataChannelAsset>(FSoftObjectPath(TEXT("/WaterAdvanced/Niagara/DataChannels/NDC_ShallowWater.NDC_ShallowWater")));
	DefaultOceanPatchNiagaraSystem = TSoftObjectPtr<UNiagaraSystem>(FSoftObjectPath(TEXT("/WaterAdvanced/Niagara/Systems/Grid2D_OceanPatch.Grid2D_OceanPatch")));
	WaterMPC = TSoftObjectPtr<UMaterialParameterCollection>(FSoftObjectPath(TEXT("/Water/Materials/MPC/MPC_Water.MPC_Water")));	
}

FName UShallowWaterSettings::GetCategoryName() const
{
	return FName("Plugins");
}
