// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShallowWaterSubsystem.h"

#include "ShallowWaterSettings.h"
#include "NiagaraComponent.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAccessor.h"
#include "NiagaraDataInterfaceArrayFunctionLibrary.h"
#include "NiagaraFunctionLibrary.h"
#include "WaterBodyActor.h"
#include "WaterSubsystem.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Kismet/GameplayStatics.h"
#include "Kismet/KismetMaterialLibrary.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "Engine/GameInstance.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/Character.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/SpectatorPawn.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Engine/AssetManager.h"
#include "Engine/Texture2D.h"
#include "EngineUtils.h"

//UE_DISABLE_OPTIMIZATION_SHIP

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShallowWaterSubsystem)

int32 GSWEnabled = 1;
static FAutoConsoleVariableRef CVarSWEnabled(
	TEXT("r.ShallowWater.Enabled"),
	GSWEnabled,
	TEXT("Should create ShallowWaterSubsystem at all"),
	ECVF_Scalability
);

float GSWRemainActiveForSeconds = 15.f;
static FAutoConsoleVariableRef CVarSWRemainActiveForSeconds(
	TEXT("r.ShallowWater.FadeOutWait"),
	GSWRemainActiveForSeconds,
	TEXT("If not any kind of collision or collision tracker is active, how long the simulation continues to be active to wait out the ripples")
);

float GSWImpactTrackerActiveForSeconds = 5.f;
static FAutoConsoleVariableRef CVarImpactTrackerActiveForSeconds(
	TEXT("r.ShallowWater.CollisionTracker.ImpactTrackerActiveForSeconds"),
	GSWImpactTrackerActiveForSeconds,
	TEXT("How long impacts (e.g. bullets hit water) are tracked, keeping the sim active") 
);

int32 GSWDrawWaterSurfaceProjection = 0;
static FAutoConsoleVariableRef CVarSWDrawWaterSurfaceProjection(
	TEXT("r.ShallowWater.DrawSurfaceProjection"),
	GSWDrawWaterSurfaceProjection,
	TEXT("")
);

int32 GSWDebugRender = 0;
static FAutoConsoleVariableRef CVarSWDebugRender(
	TEXT("r.ShallowWater.DebugRender"),
	GSWDebugRender,
	TEXT("")
);

int32 GSWUseWaterInfoTexture = 1;
static FAutoConsoleVariableRef CVarSWUseWaterInfoTexture(
	TEXT("r.ShallowWater.UseWaterInfoTexture"),
	GSWUseWaterInfoTexture,
	TEXT("")
);

int32 GSWUseFullVehiclePhysicsAssets = 1;
static FAutoConsoleVariableRef CVarSWUUseFullVehiclePhysicsAssetse(
	TEXT("r.ShallowWater.UseFullVehiclePhysicsAssets"),
	GSWUseFullVehiclePhysicsAssets,
	TEXT("")
);

FName UShallowWaterSubsystem::ColliderComponentTag = FName("RigidMesh_ShallowWaterCollider");

void FShallowWaterCollisionTracker_Actor::GetOverlappingWaterBodies(TSet<AWaterBody*>& WaterBodies) const
{
	if(const AActor* ActorPointer = CollisionActor.Get())
	{
    	TSet<AActor*> OverlappingActors;
		ActorPointer->GetOverlappingActors(OverlappingActors, AWaterBody::StaticClass());
		for (AActor* WaterBody : OverlappingActors)
		{
			WaterBodies.Add(CastChecked<AWaterBody>(WaterBody));
		}
	}
}

UShallowWaterSubsystem::UShallowWaterSubsystem()
{
}

void UShallowWaterSubsystem::PostInitialize()
{
	Super::PostInitialize();

	// Register default PA Proxies before all other GFP chimes in
	Settings = GetMutableDefault<UShallowWaterSettings>();
	if (!Settings)
	{
		ensureMsgf(false, TEXT("UShallowWaterSubsystem::PostInitialize() - UShallowWaterSettings is not valid"));
	}
}

void UShallowWaterSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);

	TArray<ULocalPlayer*> LocalPlayers = InWorld.GetGameInstance() != nullptr ? InWorld.GetGameInstance()->GetLocalPlayers() : TArray<ULocalPlayer*>();
	
	// we don't support split screen
	if (LocalPlayers.Num() == 1)
	{
		if (ULocalPlayer* const LocalPlayer = LocalPlayers[0])
		{
			APlayerController* Controller = LocalPlayer->GetPlayerController(&InWorld);
			if (Controller)
			{
				OnLocalPlayerControllerBecomesValid(Controller);
			}
			LocalPlayer->OnPlayerControllerChanged().AddUObject(this, &UShallowWaterSubsystem::OnLocalPlayerControllerBecomesValid);
		}
	}
	else if (LocalPlayers.Num() > 1)
	{
		UE_LOGF(LogShallowWater, Warning, "Shallow Water Simulation is disabled during splitscreen");
	}
}

bool UShallowWaterSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!GSWEnabled)
	{
		return false;
	}
	
	if (!FApp::CanEverRender() || IsRunningDedicatedServer())
	{
		return false;
	}

	// IsRunningDedicatedServer() is a static check and doesn't work in PIE "As Client' mode where both a server and a client are run
	if (UWorld* World = Cast<UWorld>(Outer))
	{
		if (World->IsNetMode(NM_DedicatedServer))
		{
			return false;
		}
	}
	
	return Super::ShouldCreateSubsystem(Outer);
}

void UShallowWaterSubsystem::Tick(float DeltaTime)
{	
	Super::Tick(DeltaTime);
	
	if (!IsShallowWaterInitialized())
	{
		return;
	}

	UpdateActivePawns();
	UpdateCollisionForPendingContexts();

	/*
	 **** Ticking order ****
	 *
	 * Niagara CPU Script and Pawns (order can be mixed)
	 *
	 * Subsystem
	 *     - ThisClass::UpdateGridMovement()
	 *     - Set SimCenter to water materials
	 *
	 * Single Layer Water shader
	 */
	// Leave the sim center at previous water body Z level untouched, if owner character is not in water
	TSet<AWaterBody*> WaterBodies = GetAllOverlappingWaterBodiesAndUpdateCollisionTrackers();
	if (!WaterBodies.IsEmpty())
	{
		UpdateOverlappingWaterBodiesHistory(WaterBodies.Array());
	
		for (AWaterBody* Water : WaterBodies)
		{
			if (Water == nullptr)
			{
				continue;
			}

			// loop over the water bodies for the current water zone and make sure each one has all of the materials setup with 
			// references to the simulation texture(s)
			if (UWaterBodyComponent* WaterBodyComp = Water->GetWaterBodyComponent())
			{
				if (AWaterZone* WaterZone = WaterBodyComp->GetWaterZone())
				{
					// Update all water bodies in the water zone only if it hasn't already been done
					if (!WaterZonesWithMaterialsInitialized.Find(WaterZone))
					{
						WaterZone->ForEachWaterBodyComponent([this](UWaterBodyComponent* WaterBodyComponent)
							{
								TryUpdateWaterBodyMIDParameters(WaterBodyComponent);

								return true;
							}
						);

						WaterZonesWithMaterialsInitialized.Add(WaterZone);
					}
				}
			}
		}
	}

	// We might miss some impacts because the Niagara System hasn't been activated in time.
	// Note that we do this one frame after Niagara System activation otherwise it won't work
	// due to reset tick logic in Niagara skipping the main simulation loop
	if (bFlushPendingImpactsNextTick)
	{		
		FlushPendingImpacts();
	}
	bFlushPendingImpactsNextTick = false;

	if (ShouldSimulateThisFrame())
	{
		UpdateGridMovement();
		if (!ShallowWaterNiagaraSimulation->IsActive())
		{
			ShallowWaterNiagaraSimulation->Activate(false);

			bFlushPendingImpactsNextTick = true;
		}
	}
	else
	{
		if (ShallowWaterNiagaraSimulation->IsActive())
		{
			ShallowWaterNiagaraSimulation->Deactivate();
		}
	}

	ClearTickCache();
}

TStatId UShallowWaterSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UShallowWaterSubsystem, STATGROUP_Tickables);
}

void UShallowWaterSubsystem::InitializeShallowWater()
{
	if (!ensure(WeakPlayerController.IsValid()))
	{
		UE_LOGF(LogShallowWater, Warning, "PlayerController is invalid during initialization");
		return;
	}

	// another check to make sure we don't simulate when we have split screen active
	if (WeakPlayerController->GetSplitscreenPlayerCount() > 1)
	{
		UE_LOGF(LogShallowWater, Warning, "Shallow Water Simulation is disabled during splitscreen");
		return;
	}

	if(!IsShallowWaterAllowedToInitialize())
	{
		return;
	}

	// async load the ShallowWater MPC and NS
	if (!bInitializationAsyncLoadsAttempted)
	{
		bInitializationAsyncLoadsAttempted = true;

		Settings = GetMutableDefault<UShallowWaterSettings>();

		TArray< FSoftObjectPath> ObjectsToLoad;
		ObjectsToLoad.Add(Settings->WaterMPC.ToSoftObjectPath());
		ObjectsToLoad.Add(Settings->DefaultShallowWaterNiagaraSimulation.ToSoftObjectPath());
		ObjectsToLoad.Add(Settings->DefaultShallowWaterCollisionNDC.ToSoftObjectPath());
		ObjectsToLoad.Add(Settings->PhysicsAssetProxiesDataAsset.ToSoftObjectPath());

		UAssetManager::GetStreamableManager().RequestAsyncLoad(ObjectsToLoad,
			FStreamableDelegate::CreateWeakLambda(this, [this]()
				{
					// continue with initialization after MPC and NS are loaded
					InitializeShallowWater();
				})
		);

		return;
	}
	
	MPC = Settings->WaterMPC.Get();
	if (MPC == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterSubsystem::InitializeShallowWater() - MPC cannot be loaded. Make sure it's set in ShallowWater Settings.");
		return;
	}

	/*
	 * From here, non-spectator PlayerPawn might not be available if the game is a replay
	 * So we relies on  GetTheMostRelevantPlayerPawn();
	 */
	const APawn* CursorPawn = GetTheMostRelevantPlayerPawn();
	if (CursorPawn == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "Could not find CursorPawn during initialization");
		return;
	}

	// async load the NS, then create the actor
	UNiagaraSystem* ShallowWaterTemplate = Settings->DefaultShallowWaterNiagaraSimulation.Get();
	if (ShallowWaterTemplate == nullptr)
	{		
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterSubsystem::InitializeShallowWater() - Couldn't find ShallowWater template in settings");
		return;
	}

	// async load the NS, then create the actor
	UNiagaraDataChannelAsset* ShallowWaterCollisionNDC = Settings->DefaultShallowWaterCollisionNDC.Get();
	if (ShallowWaterCollisionNDC == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterSubsystem::InitializeShallowWater() - Couldn't find ShallowWater collision NDC in settings");
		return;
	}

	const FVector SpawnLocation =  CursorPawn->GetActorLocation();
	ShallowWaterNiagaraSimulation = UNiagaraFunctionLibrary::SpawnSystemAtLocation(GetWorld(), ShallowWaterTemplate,
		SpawnLocation, FRotator::ZeroRotator, FVector::OneVector, false,
		false,
		ENCPoolMethod::None, false);

	if (ShallowWaterNiagaraSimulation == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "UShallowWaterSubsystem::InitializeShallowWater() - ShallowWaterNiagaraSystem spawn failed");
		return;
	}
	/*
	 * Initialize succeeds
	 */

	bIsShallowWaterInitialized = true;
	
	WaterZonesWithMaterialsInitialized.Empty();
	
	CreateRTs();
	
	ShallowWaterNiagaraSimulation->SetUsingAbsoluteRotation(true);
	ShallowWaterNiagaraSimulation->Activate();
		
	InitializeParameters();

	for (TWeakObjectPtr<AWaterBody> WeakWaterBody : PendingWaterBodiesToSetMIDOnInitialize)
	{
		if (AWaterBody* WaterBody = WeakWaterBody.Get())
		{
			TryUpdateWaterBodyMIDParameters(WaterBody->GetWaterBodyComponent());
		}
	}
	PendingWaterBodiesToSetMIDOnInitialize.Empty();

	PendingImpacts.Reset();
	
	// Register default PA Proxies before all other GFP chimes in
	if (Settings->PhysicsAssetProxiesDataAsset.IsValid())
	{
		RegisterPhysicsAssetProxiesDataAsset(Settings->PhysicsAssetProxiesDataAsset.Get());
	}
	else
	{
		UE_LOGF(LogShallowWater, Log, "UShallowWaterSubsystem::InitializeShallowWater() - UShallowWaterSettings::PhyicsAssetProxiesDataAsset is not valid");	
	}

	UE_LOGF(LogShallowWater, Log, "UShallowWaterSubsystem::InitializeShallowWater() finished successfully");
}

bool UShallowWaterSubsystem::IsShallowWaterAllowedToInitialize() const
{
	return false;
}

bool UShallowWaterSubsystem::IsShallowWaterInitialized() const
{
	return bIsShallowWaterInitialized;
}

APawn* UShallowWaterSubsystem::GetNonSpectatorPawnFromWeakController() const
{
	const APlayerController* Controller = WeakPlayerController.Get();
	if (Controller)
	{
		APawn* Pawn = Controller->GetPawn();
		if (Pawn && !Pawn->GetClass()->IsChildOf(ASpectatorPawn::StaticClass()))
		{
			return Pawn; 
		}
	}
	return nullptr;
}

TOptional<FVector> UShallowWaterSubsystem::GetCameraLocationFromWeakController() const
{
	TOptional<FVector> Result;
	const APlayerController* Controller = WeakPlayerController.Get();
	if (Controller)
	{
		const TObjectPtr<APlayerCameraManager> CamMan = Controller->PlayerCameraManager;
		if (CamMan)
		{
			Result = CamMan->GetCameraLocation();
			return Result;
		}
	}
	return Result;
}

APawn* UShallowWaterSubsystem::GetTheMostRelevantPlayerPawn() const
{
	if (bTickCacheValid && CachedCursorPawn)
	{
		return CachedCursorPawn;
	}

	const APlayerController* Controller = WeakPlayerController.Get();
	if (Controller)
	{
		APawn* Pawn = GetNonSpectatorPawnFromWeakController();
		if (Pawn)
		{
			return Pawn;
		}
		
		const TOptional<FVector> CamLoc = GetCameraLocationFromWeakController();
		if (CamLoc.IsSet())
		{
			APawn* BestPawn = nullptr;
			float BestDistance = FLT_MAX;
			for (APawn* TestPawn : GetPawnsInRange())
			{
			    const float Distance = FVector::Distance(CamLoc.GetValue(), TestPawn->GetActorLocation()); 
				if (Distance < BestDistance)
				{
					BestDistance = Distance;
					BestPawn = TestPawn;
				}
			}
			if (BestPawn)
			{
				return BestPawn;
			}
		}
	}
	return nullptr;
}

void UShallowWaterSubsystem::CreateRTs()
{
	if (Settings == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: Null settings.")
		return;
	}

	int32 Resolution = Settings->ShallowWaterSimParameters.ResolutionMaxAxis;

	if (Resolution <= 0)
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: Invalid Resolution Max Axis.  Defauling to 1x1 simulation.")
		Resolution = 1;
	}

	NormalRT = NewObject<UTextureRenderTarget2D>(this);
	check(NormalRT);
	NormalRT->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA16f;
	NormalRT->ClearColor = FLinearColor(0.f,0.f,0.f,0.f);
	NormalRT->bAutoGenerateMips = false;
	NormalRT->bCanCreateUAV = true;    // Niagara RT DI requires UAV. Do we need to create it here?
	NormalRT->InitAutoFormat(Resolution, Resolution);	
	NormalRT->UpdateResourceImmediate(true);	

	if (Settings->OutputVelocity)
	{
		VelocityRT = NewObject<UTextureRenderTarget2D>(this);
		check(VelocityRT);
		VelocityRT->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RG16f;
		VelocityRT->ClearColor = FLinearColor(0.f,0.f,0.f,0.f);
		VelocityRT->bAutoGenerateMips = false;
		VelocityRT->bCanCreateUAV = true;    // Niagara RT DI requires UAV. Do we need to create it here?
		VelocityRT->InitAutoFormat(Resolution, Resolution);	
		VelocityRT->UpdateResourceImmediate(true);
	}
	else
	{
		VelocityRT = nullptr;
	}
}

void UShallowWaterSubsystem::InitializeParameters()
{
	if (Settings == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: Null settings.")
		return;
	}

	if (ShallowWaterNiagaraSimulation)
	{
		ShallowWaterNiagaraSimulation->SetVariableVec2(FName("WorldGridSize"), FVector2D(GetGridSize()));
		ShallowWaterNiagaraSimulation->SetVariableInt(FName("ResolutionMaxAxis"), GetGridResolution());
		ShallowWaterNiagaraSimulation->SetVariableTextureRenderTarget(Settings->NormalRTNiagaraName, NormalRT);

		ShallowWaterNiagaraSimulation->SetVariableTextureRenderTarget(Settings->VelocityRTNiagaraName, VelocityRT);

		ShallowWaterNiagaraSimulation->SetVariableBool(FName("UseBakedSim"), false);
	}
	else
	{
		UE_LOGF(LogShallowWater, Error, "ShallowWaterComponent: No simulation found on component.")
	}
	
	if (MPC)
	{
		UKismetMaterialLibrary::SetScalarParameterValue(this, MPC,
			Settings->WorldGridSizeMPCName,
			GetGridSize());
		UKismetMaterialLibrary::SetScalarParameterValue(this, MPC,
			Settings->ResolutionMaxAxisMPCName,
			GetGridResolution());
	}
	else
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: No valid MPC found in Project Settings - Water Advanced. The simulation will work but would show preview from the Niagara renderer only.")
	}
}

void UShallowWaterSubsystem::UpdateGridMovement()
{
	if (ShallowWaterNiagaraSimulation == nullptr)
	{
		return;
	}
	
	APawn* CursorPawn = GetTheMostRelevantPlayerPawn();
	if (CursorPawn == nullptr)
	{
		return;
	}
	
	const FVector CursorPawnLocation = CursorPawn->GetActorLocation();
	
	/*
	 * Snap the sim center to player character
	 */
	// #note this query gets the closest water surface in 3D space, not a 2D topdown projection as we instinctively assumed
	// Could be better and could be worse

	float BestDistanceSqr = FLT_MAX;
	TOptional<FVector> BestWaterLocation;
	TWeakObjectPtr<AWaterBody> NewBestWaterBody;

	for (TWeakObjectPtr<AWaterBody> WeakWaterBody : LastOverlappingWaterBodies_Internal)
	{
		if (const AWaterBody* WaterBody = WeakWaterBody.Get())
		{
			const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> WaterInfo = WaterBody->GetWaterBodyComponent()->TryQueryWaterInfoClosestToWorldLocation(
				CursorPawnLocation, EWaterBodyQueryFlags::ComputeLocation);

			if (!WaterInfo.HasValue())
			{
				continue;
			}

			const FVector WaterLocation = WaterInfo.GetValue().GetWaterSurfaceLocation();
			const float ds = FVector::DistSquared(WaterLocation, CursorPawnLocation); 
			if (ds < BestDistanceSqr)
			{
				BestDistanceSqr = ds;
				BestWaterLocation = WaterLocation;
				NewBestWaterBody = WeakWaterBody;
			}
		}
	}

	if (!BestWaterLocation.IsSet())
	{
		return;
	}

	// Check to see if the best water body has changed
	bool BestWaterBodyChanged = NewBestWaterBody != BestWaterBody;
	BestWaterBody = NewBestWaterBody;

	const FVector ProjectedLocation = FVector(CursorPawnLocation.X, CursorPawnLocation.Y, 0);
	ShallowWaterNiagaraSimulation->SetWorldLocation(ProjectedLocation);
	ShallowWaterNiagaraSimulation->SetVariableFloat(FName("WaterHeightAtParent"), BestWaterLocation.GetValue().Z);

	// time left before sim is destroyed- used to attenuate waves
	const float SecondsSinceCollision = GetWorld()->GetTimeSeconds() - LastTimeOverlappingAnyWaterBody;
	const float SecondsUntilDestroyed = FMath::Max(0, GSWRemainActiveForSeconds - SecondsSinceCollision);
	ShallowWaterNiagaraSimulation->SetVariableFloat(FName("SecondsUntilDestroyed"), SecondsUntilDestroyed);

	ShallowWaterNiagaraSimulation->SetVariableBool(FName("UseDebugRender"), GSWDebugRender == 1);
	
	if (BestWaterBodyChanged && BestWaterBody != nullptr)
	{
		bool UseBakedSim = false;

		if (const UWaterBodyComponent* WaterBodyComp = BestWaterBody->GetWaterBodyComponent())
		{
			if (WaterBodyComp->UseBakedSimulationForQueriesAndPhysics())
			{
				UBakedShallowWaterSimulationComponent* BakedSim = WaterBodyComp->GetBakedShallowWaterSimulation();				
				UTexture* BakedSimTex = Cast<UTexture>(BakedSim->BakedSimulationData->BakedTexture.Get());

				if (BakedSimTex != nullptr)
				{
					ShallowWaterNiagaraSimulation->SetVariableVec3(FName("BakedWaterSimLocation"), BakedSim->BakedSimulationData->Position);
					ShallowWaterNiagaraSimulation->SetVariableVec2(FName("BakedWaterSimSize"), BakedSim->BakedSimulationData->Size);
					ShallowWaterNiagaraSimulation->SetVariableVec2(FName("BakedWaterSimRes"), 
						FVector2D(BakedSim->BakedSimulationData->NumCells.X, BakedSim->BakedSimulationData->NumCells.Y));
					ShallowWaterNiagaraSimulation->SetVariableTexture(FName("BakedWaterSimTexture"), BakedSimTex);

					UseBakedSim = true;				
				}
			}
		}
		
		ShallowWaterNiagaraSimulation->SetVariableBool(FName("UseBakedSim"), UseBakedSim);

		// update to use the water info from this water body
		bool UseWaterInfoTexture = GSWUseWaterInfoTexture == 1;
		ShallowWaterNiagaraSimulation->SetVariableBool(FName("UseWaterInfoTexture"), UseWaterInfoTexture);
		if (UseWaterInfoTexture)
		{			
			TryGetOrWaitForWaterInfoTextureFromWaterBody(BestWaterBody.Get());
		}
	}
	
#if ENABLE_DRAW_DEBUG
	if (GSWDrawWaterSurfaceProjection)
	{
		DrawDebugLine(GetWorld(), CursorPawnLocation, BestWaterLocation.GetValue(), FColor::Yellow, false, 0.5f);
		DrawDebugPoint(GetWorld(), BestWaterLocation.GetValue(), 5.f, FColor::Yellow, false, 0.5f);
	}
#endif

	/*
	 * Feed sim center to MPC
	 */
	if (MPC != nullptr)
	{
		UKismetMaterialLibrary::SetVectorParameterValue(this, MPC, Settings->GridCenterMPCName,
			FLinearColor(PreviousProjectedLocation.X, PreviousProjectedLocation.Y, 0.f, 0.f));
	}
	PreviousProjectedLocation = ProjectedLocation;
}

void UShallowWaterSubsystem::RegisterImpact(FVector ImpactPosition, FVector ImpactVelocity, float ImpactRadius)
{
	if (ShallowWaterNiagaraSimulation == nullptr)
	{
		return;
	}
	
	FHitResult OutHit;
	GetWorld()->LineTraceSingleByChannel(OutHit, ImpactPosition + FVector(0,0,10.f), ImpactPosition + FVector(0,0,-10.f),
		GetImpactCollisionChannel());
	AWaterBody* WaterBody = Cast<AWaterBody>(OutHit.GetActor());
	if (WaterBody == nullptr)
	{
		return;
	}
	Tracker_Directs.Add(FShallowWaterCollisionTracker_Direct(GetWorld()->GetTimeSeconds(), GSWImpactTrackerActiveForSeconds, WaterBody));
	
	if (!ShallowWaterNiagaraSimulation->IsActive())
	{
		// queue up impacts we missed
		PendingImpact TmpImpact;
		TmpImpact.ImpactPosition = ImpactPosition;
		TmpImpact.ImpactVelocity = ImpactVelocity;
		TmpImpact.ImpactRadius = ImpactRadius;
		PendingImpacts.Add(TmpImpact);

		return;
	}

	WriteImpactToNDC(ImpactPosition, ImpactVelocity, ImpactRadius);
}

// these impacts are a frame or two late, and we only want to update the Niagara system and not 
// perform an additional overlap test
void UShallowWaterSubsystem::FlushPendingImpacts()
{
	for (PendingImpact CurrPendingImpact : PendingImpacts)
	{
		WriteImpactToNDC(CurrPendingImpact.ImpactPosition, CurrPendingImpact.ImpactVelocity, CurrPendingImpact.ImpactRadius);
	}
	PendingImpacts.Reset();
}

void UShallowWaterSubsystem::WriteImpactToNDC(FVector ImpactPosition, FVector ImpactVelocity, float ImpactRadius)
{
	FNiagaraDataChannelSearchParameters SearchParams(ImpactPosition);
	const UNiagaraDataChannelAsset* NDC = Settings->DefaultShallowWaterCollisionNDC.Get();
	if (NDC == nullptr)
	{
		return;
	}
	
	if (UNiagaraDataChannelWriter* DCWriter = UNiagaraDataChannelLibrary::WriteToNiagaraDataChannel(ShallowWaterNiagaraSimulation, NDC, SearchParams, 1, false, true, true, TEXT("ShallowWaterWriteImpact")))
	{
		int32 Index = 0;
		DCWriter->WritePosition(TEXT("Position"), Index, ImpactPosition);
		DCWriter->WriteVector(TEXT("Velocity"), Index, ImpactVelocity);
		DCWriter->WriteFloat(TEXT("Radius"), Index, ImpactRadius);
	}
}

void UShallowWaterSubsystem::SetWaterBodyMIDParameters(AWaterBody* WaterBody)
{
	if (WaterBody)
	{
		if (IsShallowWaterInitialized())
		{
			TryUpdateWaterBodyMIDParameters(WaterBody->GetWaterBodyComponent());
		}
		else
		{
			// On BeginPlay this might not be ready yet
			PendingWaterBodiesToSetMIDOnInitialize.AddUnique(WaterBody);
		}
	}
}

void UShallowWaterSubsystem::TryUpdateWaterBodyMIDParameters(UWaterBodyComponent* WaterBodyComponent)
{
	if (WaterBodyComponent == nullptr || WaterBodyComponentsWithProperMIDParameters.Contains(WaterBodyComponent))
	{
		// UE_LOGF(LogShallowWater, Warning, "TryUpdateWaterBodyMIDParameters failed");
		return;
	}
	WaterBodyComponentsWithProperMIDParameters.Add(WaterBodyComponent);
	
	if (UMaterialInstanceDynamic* WaterMID = WaterBodyComponent->GetWaterMaterialInstance(); WaterMID)
	{
		WaterMID->SetTextureParameterValue(Settings->NormalRTMaterialName, NormalRT);
		WaterMID->SetTextureParameterValue(Settings->VelocityRTMaterialName, VelocityRT);
		WaterMID->SetScalarParameterValue(FName(TEXT("DEV_UseNewShallowWaterSubsystem")), 1.f);   // Temp before replacing OG system
	}
	if (UMaterialInstanceDynamic* WaterMID = WaterBodyComponent->GetRiverToOceanTransitionMaterialInstance(); WaterMID)
	{
		WaterMID->SetTextureParameterValue(Settings->NormalRTMaterialName, NormalRT);
		WaterMID->SetTextureParameterValue(Settings->VelocityRTMaterialName, VelocityRT);
		WaterMID->SetScalarParameterValue(FName(TEXT("DEV_UseNewShallowWaterSubsystem")), 1.f);   // Temp before replacing OG system
	}
	if (UMaterialInstanceDynamic* WaterMID = WaterBodyComponent->GetRiverToLakeTransitionMaterialInstance(); WaterMID)
	{
		WaterMID->SetTextureParameterValue(Settings->NormalRTMaterialName, NormalRT);
		WaterMID->SetTextureParameterValue(Settings->VelocityRTMaterialName, VelocityRT);
		WaterMID->SetScalarParameterValue(FName(TEXT("DEV_UseNewShallowWaterSubsystem")), 1.f);   // Temp before replacing OG system
	}
}

void UShallowWaterSubsystem::RegisterPhysicsAssetProxiesDataAsset(const UShallowWaterPhysicsAssetOverridesDataAsset* Proxies)
{
	if (Proxies == nullptr)
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: UShallowWaterPhysicsAssetOverridesDataAsset is NULL.  No vehicle interaction will be possible.")
		return;
	}
	else if (Proxies->Overrides.Num() <= 0)
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: Input UShallowWaterPhysicsAssetOverridesDataAsset: %ls has 0 entries.  No additional vehicles will be supported.", *Proxies->GetName())
		return;
	}

	for (const TTuple<FGameplayTag, FShallowWaterPhysicsAssetOverride>& Override : Proxies->Overrides)
	{
		if (RegisteredPhysicsAssetProxies.Contains(Override.Key))
		{
			UE_LOGF(LogShallowWater, Log, "Physics Asset Override in %ls is overwriting an existing Override. GameplayTag = %ls. This could be intended.",
				*Proxies->GetName(), *Override.Key.ToString())
		}
	}
	RegisteredPhysicsAssetProxies.Append(Proxies->Overrides);

	if (RegisteredPhysicsAssetProxies.Num() <= 0)
	{
		UE_LOGF(LogShallowWater, Warning, "ShallowWaterComponent: RegisteredPhysicsAssetProxies has 0 entries.  No vehicle interaction will be possible.")
		return;
	}
}

TSet<AWaterBody*> UShallowWaterSubsystem::GetAllOverlappingWaterBodiesAndUpdateCollisionTrackers() 
{
	TSet<AWaterBody*> Result = GetOverlappingWaterBodiesFromPawns();
	GetOverlappingWaterBodiesFromActorTrackersAndUpdate(Result);
	GetOverlappingWaterBodiesFromDirectTrackersAndUpdate(Result);
	return Result;
}

void UShallowWaterSubsystem::AddCollisionTrackerForActor(AActor* CollisionTrackerActor, float MaxLifespan)
{
	if(CollisionTrackerActor == nullptr)
	{
		return;
	}
		
	bool bFound = false;
	for (FShallowWaterCollisionTracker_Actor& Tracker : Tracker_Actors)
	{
		if (Tracker.CollisionActor == CollisionTrackerActor)
		{
			Tracker.TimeSpawned = GetWorld()->GetTimeSeconds();
			bFound = true;
			break;
		}
	}
	if (!bFound)
	{
		Tracker_Actors.Add(FShallowWaterCollisionTracker_Actor(GetWorld()->GetTimeSeconds(), MaxLifespan, CollisionTrackerActor));
	}
}

void UShallowWaterSubsystem::RemoveCollisionTrackerForActor(AActor* CollisionTrackerActor)
{
	if(CollisionTrackerActor == nullptr)
	{
		return;
	}

	for (int32 Index = Tracker_Actors.Num()-1; Index >= 0; Index--)
	{
		if (Tracker_Actors[Index].CollisionActor == CollisionTrackerActor)
		{
			Tracker_Actors.RemoveAt(Index);
		}
	}
}

TSet<AWaterBody*> UShallowWaterSubsystem::GetOverlappingWaterBodiesFromPawns() const
{
	TSet<AWaterBody*> Result;
	TArray<APawn*> Pawns = GetPawnsInRange();

	TSet<AActor*> OverlappingActors;
	TSubclassOf<AActor> WaterBodyClass = AWaterBody::StaticClass();

	for (const APawn* Pawn : Pawns)
	{
		Pawn->GetOverlappingActors(OverlappingActors, WaterBodyClass);
		for (AActor* WaterBody : OverlappingActors)
		{
			Result.Add(CastChecked<AWaterBody>(WaterBody));
		}
	}
	return Result;
}

void UShallowWaterSubsystem::GetOverlappingWaterBodiesFromActorTrackersAndUpdate(TSet<AWaterBody*>& WaterBodies)
{
	for (int32 Index = Tracker_Actors.Num() - 1; Index >= 0; Index--)
	{
		if (!Tracker_Actors[Index].IsValid(GetWorld()->GetTimeSeconds()))
		{
			Tracker_Actors.RemoveAt(Index);
			continue;
		}
		Tracker_Actors[Index].GetOverlappingWaterBodies(WaterBodies);
	}
}

void UShallowWaterSubsystem::GetOverlappingWaterBodiesFromDirectTrackersAndUpdate(TSet<AWaterBody*>& WaterBodies)
{
	for (int32 Index = Tracker_Directs.Num() - 1; Index >= 0; Index--)
	{
		if (!Tracker_Directs[Index].IsValid(GetWorld()->GetTimeSeconds()))
		{
			Tracker_Directs.RemoveAt(Index);
			continue;
		}
		if (AWaterBody* WaterBody = Tracker_Directs[Index].GetOverlappingWaterBody())
		{
			WaterBodies.Add(WaterBody);
		}
	}
}

void UShallowWaterSubsystem::UpdateOverlappingWaterBodiesHistory(TArray<AWaterBody*> OverlappingWaterBodies)
{
	if (!OverlappingWaterBodies.IsEmpty())
	{
		LastTimeOverlappingAnyWaterBody = GetWorld()->GetTimeSeconds();
		LastOverlappingWaterBodies_Internal.Reset();
		for (AWaterBody* WaterBody : OverlappingWaterBodies)
		{
			LastOverlappingWaterBodies_Internal.Add(WaterBody);  			
		}
	}
}

bool UShallowWaterSubsystem::ShouldSimulateThisFrame() const
{
	return (GetWorld()->GetTimeSeconds() - LastTimeOverlappingAnyWaterBody <= GSWRemainActiveForSeconds);
}

void UShallowWaterSubsystem::ClearTickCache()
{
	bTickCacheValid = false;
	CachedCursorPawn = nullptr;
	CachedPawnsInRange.Reset();
}

int32 UShallowWaterSubsystem::UpdateActivePawns()
{
	bTickCacheValid = false;

	CachedCursorPawn = GetTheMostRelevantPlayerPawn();
	if (CachedCursorPawn == nullptr)
	{
		ActivePawns.Empty();
		return 0;
	}

	FVector CursorPawnLocation = CachedCursorPawn->GetActorLocation();
	
	// Use an array of hard pointers to speed up processing
	TArray<APawn*> ValidActivePawns;
	ValidActivePawns.Reserve(ActivePawns.Num());

	FVector::FReal DistanceSquared = FMath::Square(GetColliderMaxRange());
	
	ActivePawns.RemoveAll([&](TWeakObjectPtr<APawn> WeakPawn)
	{
		APawn* Pawn = WeakPawn.Get();
		if (Pawn && (Pawn->GetActorLocation() - CursorPawnLocation).SquaredLength() <= DistanceSquared)
		{
			// Add to valid list if it's within range
			ValidActivePawns.Add(Pawn);
			return false;
		}
		// Remove from weak pointer list
		return true;
	});

	// Update the cached pawns here for later
	CachedPawnsInRange = GetPawnsInRange(CursorPawnLocation, false);

	const int32 SlotsLeft = Settings->MaxActivePawnNum - ValidActivePawns.Num();
	int32 NewPawnsAdded = 0;
	if (SlotsLeft > 0)
	{
		// Get pawns nearby who become relevant this frame
		TArray<APawn*> RelevantPawns = CachedPawnsInRange;
				
		// Remove ones already in active list
		RelevantPawns.RemoveAllSwap([&](APawn* Pawn)
		{
			return ValidActivePawns.Contains(Pawn);
		}, EAllowShrinking::No);

		// Sort pawn candidates
		RelevantPawns.Sort([CursorPawnLocation] (const APawn& Left, const APawn& Right)
		{
			// Compare 3D distance since the other player right above you, although might be pretty far away, are going to drop on your face fast
			// Thus prioritized over the player in water but a little further away
			return (Left.GetActorLocation() - CursorPawnLocation).SquaredLength()
			< (Right.GetActorLocation() - CursorPawnLocation).SquaredLength();
		});

		NewPawnsAdded = FMath::Min(SlotsLeft, RelevantPawns.Num());
		for (int32 Index = 0; Index < NewPawnsAdded; Index++)
		{
			// Update both weak and strong lists
			ActivePawns.Add(RelevantPawns[Index]);
			ValidActivePawns.Add(RelevantPawns[Index]);
		}

#if ENABLE_DRAW_DEBUG
		if (Settings->bVisualizeActivePawn)
		{
			// Use Niagara to draw debug should be better. If yes delete this
		}
#endif
	}

	// Update pending collision contexts for new list
	for (APawn* PawnPtr : ValidActivePawns)
	{
		TOptional<FShallowWaterCollisionContext> Context = GetCollisionContextFromPawn(PawnPtr);
		if (Context.IsSet())
		{
			// AddUnique() since multiple pawns can return the same Context. E.g. a multi-seat vehicle
			PendingContexts.AddUnique(Context.GetValue());
		}
	}
	
	// Update tick cache, ResetTickCache must be called before the calling function returns to ensure safety
	bTickCacheValid = true;

	return NewPawnsAdded;
}

TOptional<FShallowWaterCollisionContext> UShallowWaterSubsystem::GetCollisionContextFromPawn(APawn* InPawn) const
{
	TOptional<FShallowWaterCollisionContext> Result;
	USkeletalMeshComponent* Component = nullptr;
	const ACharacter* Character = Cast<ACharacter>(InPawn);
	if (Character)
	{
		Component = Character->GetMesh();
	}
	else
	{
		Component = InPawn->GetComponentByClass<USkeletalMeshComponent>();
	}
	if (Component)
	{
		Result = FShallowWaterCollisionContext(EShallowWaterCollisionContextType::Pawn, Component); 
	}
	return Result;
}

void UShallowWaterSubsystem::CleanUpVehicleCollisionProxies()
{
	TArray<TTuple<const FShallowWaterCollisionContext, USkeletalMeshComponent*>> EntriesToRemove;
	for (TTuple<const FShallowWaterCollisionContext, USkeletalMeshComponent*> Entry : VehicleCollisionProxies)
	{
		// Shotgun approach to cover all weird possibilities
		// E.g. if the Vehicle component is destroyed but not the owning actor somehow
		if ( Entry.Key.WeakComponent.Get() == nullptr ||
			Entry.Value == nullptr || Entry.Value->IsBeingDestroyed())
		{
			EntriesToRemove.Add(Entry);
		}
	}
	for (TTuple<const FShallowWaterCollisionContext, USkeletalMeshComponent*> Entry : EntriesToRemove)
	{
		DisableCollisionForVehicle(Entry.Key);
	}
}

void UShallowWaterSubsystem::UpdateCollisionForPendingContexts()
{
	CleanUpVehicleCollisionProxies();
	
	// Find components that need their collision disabled
	for (const FShallowWaterCollisionContext& Context : PreviousContexts)
	{
		if (!PendingContexts.Contains(Context))
		{
			DisableCollisionForContext(Context);
		}
	}

	// Find components that don't already exist
	for (const FShallowWaterCollisionContext& Context : PendingContexts)
	{
		if (!PreviousContexts.Contains(Context))
		{
			EnableCollisionForContext(Context);
		}
	}

	PreviousContexts = PendingContexts;
	PendingContexts.Reset();
}

void UShallowWaterSubsystem::EnableCollisionForContext(const FShallowWaterCollisionContext& Context)
{
	USkeletalMeshComponent* Component = Context.WeakComponent.Get();
	if (Component == nullptr)
	{		
		UE_LOGF(LogShallowWater, Warning, "EnableCollisionForContext() - Context Component is nullptr");
		return;
	}
	switch (Context.Type)
	{
	case EShallowWaterCollisionContextType::Pawn:
		
		Component->ComponentTags.AddUnique(ColliderComponentTag);
		break;
		
	case EShallowWaterCollisionContextType::Vehicle:
		{
		// Spawn proxy SKM, attach to vehicle
		const FString BaseName = TEXT("FluidsimCollisionProxy");
		const FName CompName = MakeUniqueObjectName(this, USkeletalMeshComponent::StaticClass(), FName(*BaseName));
		USkeletalMeshComponent* ProxyComp = NewObject<USkeletalMeshComponent>(Component->GetOwner(), USkeletalMeshComponent::StaticClass(), CompName);
		ProxyComp->SetSkeletalMeshAsset(Component->GetSkeletalMeshAsset());

		// Spawn an empty dummy SKM component to be used as Collision 
		// Apply PhysicsAsset override if defined in the data asset
		// #todo confirm if SetLeaderPoseComponent actually works if Mesh is set to empty #PLAY-29387
		UPhysicsAsset* PhysicsAssetOverride = nullptr;
		bool IsSet = false;
		if (!RegisteredPhysicsAssetProxies.IsEmpty())
		{
			const FGameplayTagContainer VehicleTags = GetVehicleTags(Context);

			const FShallowWaterPhysicsAssetOverride* FoundOverride = nullptr;
			for (FGameplayTag Tag : VehicleTags)
			{
				FoundOverride = RegisteredPhysicsAssetProxies.Find(Tag);
				if (FoundOverride)
				{
					break;
				}
			}
			if (FoundOverride)
			{
				IsSet = true;

				TSoftObjectPtr<UPhysicsAsset> TmpPhysicsAsset = FoundOverride->PhysicsAsset;

				UAssetManager::GetStreamableManager().RequestAsyncLoad(TmpPhysicsAsset.ToSoftObjectPath(),
					FStreamableDelegate::CreateWeakLambda(this, [this, TmpPhysicsAsset, ProxyComp, Context, Component]()
						{								
							USkeletalMeshComponent* ContextComponent = Context.WeakComponent.Get();
							if (ContextComponent && ContextComponent->GetOwner() && TmpPhysicsAsset.IsValid())
							{
								UPhysicsAsset* PhysicsAssetOverride = TmpPhysicsAsset.Get();

								ProxyComp->SetPhysicsAsset(PhysicsAssetOverride);

								ProxyComp->ComponentTags.AddUnique(ColliderComponentTag);

								ProxyComp->SetupAttachment(ContextComponent->GetOwner()->GetRootComponent());
								ProxyComp->SetWorldTransform(ContextComponent->GetComponentTransform());
								ProxyComp->SetLeaderPoseComponent(ContextComponent);
								ProxyComp->SetVisibility(false);
								ProxyComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
								ProxyComp->RegisterComponent();

								VehicleCollisionProxies.Add(Context, ProxyComp);
							}
						})
				);
			}			
		}
		if (!IsSet)
		{
			if (GSWUseFullVehiclePhysicsAssets)
			{
				Component->ComponentTags.AddUnique(ColliderComponentTag);
			}
			else
			{
			 	UE_LOGF(LogShallowWater, Warning, "EnableCollisionForContext() - Vehicle will not have collisions because no physics asset override was found");
			}
		}
		break;
		}
		
	default:
		;
	}
}

void UShallowWaterSubsystem::DisableCollisionForContext(const FShallowWaterCollisionContext& Context)
{
	if (!Context.IsValidAndAlive())
	{
		switch (Context.Type)
		{
		case EShallowWaterCollisionContextType::Pawn:
			break;
		case EShallowWaterCollisionContextType::Vehicle:
			DisableCollisionForVehicle(Context);
			break;
		default:
			;
		}
		// Otherwise no care needed because the component is dead
		return;
	}

	switch (Context.Type)
	{
	case EShallowWaterCollisionContextType::Pawn:
		if (USkeletalMeshComponent* Component = Context.WeakComponent.Get())
		{
			Component->ComponentTags.Remove(ColliderComponentTag);
		}
		break;
	case EShallowWaterCollisionContextType::Vehicle:
		DisableCollisionForVehicle(Context);
		break;
	default:
		;
	}
}

void UShallowWaterSubsystem::DisableCollisionForVehicle(const FShallowWaterCollisionContext& Context)
{
	if (Context.Type != EShallowWaterCollisionContextType::Vehicle)
	{
		return;
	}

	TObjectPtr<USkeletalMeshComponent>* ProxyPointer = VehicleCollisionProxies.Find(Context);
	if (ProxyPointer == nullptr)
	{		
		return;
	}
	
	USkeletalMeshComponent* Proxy = ToRawPtr(*ProxyPointer);
	if (Proxy && !Proxy->IsBeingDestroyed())
	{
		Proxy->DestroyComponent();
	}
	VehicleCollisionProxies.Remove(Context);
}

float UShallowWaterSubsystem::GetColliderMaxRange() const
{
	return GetGridSize() * 0.5f;
}

TArray<APawn*> UShallowWaterSubsystem::GetPawnsInRange(const bool bShouldSortBySignificance) const
{
	if (bTickCacheValid && !bShouldSortBySignificance)
	{
		return CachedPawnsInRange;
	}

	const APlayerController* Controller = WeakPlayerController.Get();
	APawn* CursorPawn = nullptr;
	if (Controller)
	{
		CursorPawn = GetNonSpectatorPawnFromWeakController();
	}
	
	FVector ObservingLocation;
	if (CursorPawn)
	{
		ObservingLocation = CursorPawn->GetActorLocation();
	}
	else
	{
		TOptional<FVector> CamLoc = GetCameraLocationFromWeakController();
		if (CamLoc.IsSet())
		{
			ObservingLocation = CamLoc.GetValue();
		}
		else
		{
			// Not even camera location is available. Game state not valid.
			return TArray<APawn*>();
		}
	}

	return GetPawnsInRange(ObservingLocation, bShouldSortBySignificance);
}

TArray<APawn*> UShallowWaterSubsystem::GetPawnsInRange(const FVector ObservingLocation, const bool bShouldSortBySignificance) const
{
	UWorld* World = GetWorld();

	FVector::FReal DistanceSquared = FMath::Square(GetColliderMaxRange());
	TArray<APawn*> Results;
	for (TActorIterator<APawn> It(World, APawn::StaticClass()); It; ++It)
	{
		APawn *Pawn = *It;
		if ((ObservingLocation - Pawn->GetActorLocation()).SquaredLength() <= DistanceSquared)
		{
			Results.Add(Pawn);
		}
	}

	if (bShouldSortBySignificance)
	{
		Results.Sort([ObservingLocation](APawn& PawnA, APawn& PawnB)
		{
			return (ObservingLocation - PawnA.GetActorLocation()).SquaredLength() < (ObservingLocation - PawnB.GetActorLocation()).SquaredLength();
		});
	}
	
	return Results;
}

void UShallowWaterSubsystem::SetSimParametersForWaterZone(AWaterZone *WaterZone)
{
	// The following index assume that there is no split screen support and will request the position of the first player's water view.
	const int32 PlayerIndex = 0;
	FVector ZoneLocation;
	WaterZone->GetDynamicWaterInfoCenter(PlayerIndex, ZoneLocation);

	const FVector2D ZoneExtent = FVector2D(WaterZone->GetDynamicWaterInfoExtent());
	const FVector2D WaterHeightExtents = FVector2D(WaterZone->GetWaterHeightExtents());
	const float GroundZMin = WaterZone->GetGroundZMin();

	ShallowWaterNiagaraSimulation->SetVariableVec2(FName("WaterZoneLocation"), FVector2D(ZoneLocation));
	ShallowWaterNiagaraSimulation->SetVariableVec2(FName("WaterZoneExtent"), ZoneExtent);
	ShallowWaterNiagaraSimulation->SetVariableInt(FName("WaterZoneIdx"), WaterZone->GetWaterZoneIndex());
}

bool UShallowWaterSubsystem::TryGetOrWaitForWaterInfoTextureFromWaterBody(AWaterBody *CurrentWaterBody)
{    
	if (const UWaterBodyComponent* WaterBodyComp = CurrentWaterBody->GetWaterBodyComponent())
	{
	    if (AWaterZone* WaterZone = WaterBodyComp->GetWaterZone())
	    {
	    	const TObjectPtr<UTextureRenderTarget2DArray> NewWaterInfoTexture = WaterZone->WaterInfoTextureArray;

	    	if (NewWaterInfoTexture == nullptr)
	    	{
	    		WaterZone->GetOnWaterInfoTextureArrayCreated().RemoveDynamic(this, &UShallowWaterSubsystem::OnWaterInfoTextureArrayCreated);
	    		WaterZone->GetOnWaterInfoTextureArrayCreated().AddDynamic(this, &UShallowWaterSubsystem::OnWaterInfoTextureArrayCreated);
	    	}
	    	else
	    	{
	    		OnWaterInfoTextureArrayCreated(NewWaterInfoTexture);
	    	}

			if (ShallowWaterNiagaraSimulation)
			{
				SetSimParametersForWaterZone(WaterZone);
			}
			else
			{
				ensureMsgf(false, TEXT("UShallowWaterSubsystem::TryGetOrWaitForWaterInfoTextureFromWaterBodies was called with NULL ShallowWaterNiagaraSimulation"));
				return false;
			}
	    	
			// Currently there can only be one unique WaterInfoTexture
	    	return true;
	    }
	}   

	return false;
}

void UShallowWaterSubsystem::TryGetOrWaitForWaterInfoTextureFromWaterBodies(TSet<AWaterBody*> CurrentWaterBodies)
{
    for (const AWaterBody* CurrentWaterBody : CurrentWaterBodies)
    {
	    if (const UWaterBodyComponent* WaterBodyComp = CurrentWaterBody->GetWaterBodyComponent())
	    {
	    	if (AWaterZone* WaterZone = WaterBodyComp->GetWaterZone())
	    	{
	    		const TObjectPtr<UTextureRenderTarget2DArray> NewWaterInfoTexture = WaterZone->WaterInfoTextureArray;

	    		if (NewWaterInfoTexture == nullptr)
	    		{
	    			WaterZone->GetOnWaterInfoTextureArrayCreated().RemoveDynamic(this, &UShallowWaterSubsystem::OnWaterInfoTextureArrayCreated);
	    			WaterZone->GetOnWaterInfoTextureArrayCreated().AddDynamic(this, &UShallowWaterSubsystem::OnWaterInfoTextureArrayCreated);
	    		}
	    		else
	    		{
	    			OnWaterInfoTextureArrayCreated(NewWaterInfoTexture);
	    		}

				if (ShallowWaterNiagaraSimulation)
				{
					SetSimParametersForWaterZone(WaterZone);
				}
				else
				{
					ensureMsgf(false, TEXT("UShallowWaterSubsystem::TryGetOrWaitForWaterInfoTextureFromWaterBodies was called with NULL ShallowWaterNiagaraSimulation"));
					return;
				}

	    		// Currently there can only be one unique WaterInfoTexture
	    		break;
	    	}
	    }
    }
}

void UShallowWaterSubsystem::OnWaterInfoTextureArrayCreated(const UTextureRenderTarget2DArray* InWaterInfoTexture)
{
	if(InWaterInfoTexture == nullptr)
	{
		ensureMsgf(false, TEXT("UShallowWaterSubsystem::OnWaterInfoTextureCreated was called with NULL WaterInfoTexture"));
		return;
	}
	WaterInfoTexture = InWaterInfoTexture;
	if (ShallowWaterNiagaraSimulation)
	{
		UTexture* WITTextureArray = Cast<UTexture>(const_cast<UTextureRenderTarget2DArray*>(WaterInfoTexture.Get()));
		if (WITTextureArray == nullptr)
		{
			ensureMsgf(false, TEXT("UShallowWaterSubsystem::OnWaterInfoTextureCreated was called with Water Info Texture that isn't valid"));
			return;
		}

		ShallowWaterNiagaraSimulation->SetVariableTexture(FName("WaterInfoTexture"), WITTextureArray);
	}
	else
	{
		ensureMsgf(false, TEXT("UShallowWaterSubsystem::OnWaterInfoTextureCreated was called with NULL ShallowWaterNiagaraSimulation"));
		return;
	}
}

void UShallowWaterSubsystem::OnLocalPlayerControllerBecomesValid(APlayerController* InPlayerController)
{
	if (InPlayerController == nullptr)
	{
		UE_LOGF(LogShallowWater, Log, "OnLocalPlayerControllerBecomesValid() returned nullptr")
		return;
	}
	
	UE_LOGF(LogShallowWater, Log, "OnLocalPlayerControllerBecomesValid() returned PC: %ls", *InPlayerController->GetFullName())
	WeakPlayerController = InPlayerController;
	if (APawn* const Pawn = InPlayerController->GetPawn())
	{
		OnLocalPlayerPawnBecomesValid(nullptr, Pawn);
	}
	InPlayerController->OnPossessedPawnChanged.RemoveDynamic(this, &ThisClass::OnLocalPlayerPawnBecomesValid);
	InPlayerController->OnPossessedPawnChanged.AddDynamic(this, &ThisClass::OnLocalPlayerPawnBecomesValid);
}

void UShallowWaterSubsystem::OnLocalPlayerPawnBecomesValid(APawn* OldPawn, APawn* NewPawn)
{
	if (NewPawn == nullptr)
	{
		UE_LOGF(LogShallowWater, Log, "OnLocalPlayerPawnBecomesValid() returned nullptr")
		return;
	}
	
	UE_LOGF(LogShallowWater, Log, "OnLocalPlayerPawnBecomesValid() returned Pawn: %ls", *NewPawn->GetFullName())
	// #todo Should re-initialize if pawn changed, or uninitialize if pawn lost 
	if (!IsShallowWaterInitialized())
	{
		InitializeShallowWater();
	}
	else
	{
		UE_LOGF(LogShallowWater, Log, "OnLocalPlayerPawnBecomesValid called but subsystem is already initialized.");
	}
}

//UE_ENABLE_OPTIMIZATION_SHIP
