// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterAdvanced.h"
#include "ShallowWaterRiverDetails.h"
#include "ShallowWaterRiverActor.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "PropertyCustomizationHelpers.h"
#endif

#define LOCTEXT_NAMESPACE "FWaterAdvancedModule"

void FWaterAdvancedModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
#if WITH_EDITOR
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(
		UShallowWaterRiverComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FShallowWaterRiverDetails::MakeInstance));
#endif
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("WaterAdvanced"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/Experimental/WaterAdvanced"), PluginShaderDir);
}

void FWaterAdvancedModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.xture
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FWaterAdvancedModule, WaterAdvanced)