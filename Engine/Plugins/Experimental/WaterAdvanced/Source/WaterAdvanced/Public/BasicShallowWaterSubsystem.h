// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShallowWaterSubsystem.h"
#include "BasicShallowWaterSubsystem.generated.h"

#define UE_API WATERADVANCED_API


UCLASS(MinimalAPI, Blueprintable, Transient)
class UBasicShallowWaterSubsystem : public UShallowWaterSubsystem
{
	GENERATED_BODY()

public:
	UE_API UBasicShallowWaterSubsystem();

	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual bool IsShallowWaterAllowedToInitialize() const override;	
};

#undef UE_API
