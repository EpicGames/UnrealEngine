// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "ShallowWaterSettings.h"
#include "FFTOceanPatchSubsystem.generated.h"

#define UE_API WATERADVANCED_API

class UNiagaraComponent;
class UTextureRenderTarget2D;

UCLASS(MinimalAPI, Blueprintable, Transient)
class UFFTOceanPatchSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UE_API UFFTOceanPatchSubsystem();

	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void PostInitialize() override;
	UE_API virtual TStatId GetStatId() const override;
	
	UE_API TObjectPtr<UTextureRenderTarget2D> GetOceanNormalRT(UWorld* World);
	TObjectPtr<UNiagaraComponent> GetOceanSystem() { return FFTOceanSystem; }
protected:
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TObjectPtr<UShallowWaterSettings> Settings;
private:
	TObjectPtr<UNiagaraComponent> FFTOceanSystem;
	TObjectPtr<UTextureRenderTarget2D> OceanNormalRT;
};

#undef UE_API
