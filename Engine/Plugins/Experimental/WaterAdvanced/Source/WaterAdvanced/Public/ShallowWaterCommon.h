// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShallowWaterCommon.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogShallowWater, Log, All);

class UPhysicsAsset;

USTRUCT(BlueprintType)
struct FShallowWaterSimParameters 
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Shallow Water")
	float WorldGridSize = 2000.f;
	UPROPERTY(EditAnywhere, Category = "Shallow Water")
	int32 ResolutionMaxAxis = 512;	
};

USTRUCT(BlueprintType)
struct FShallowWaterPhysicsAssetOverride
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, Category = "Shallow Water")
	TSoftObjectPtr<UPhysicsAsset> PhysicsAsset;
};
