// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "ShallowWaterCommon.h"
#include "GameplayTagContainer.h"
#include "ShallowWaterPhysicsAssetOverridesDataAsset.generated.h"

class USkeletalMesh;

/**
 * 
 */
UCLASS(MinimalAPI, Blueprintable)
class UShallowWaterPhysicsAssetOverridesDataAsset : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Shallow Water")
	TMap<FGameplayTag, FShallowWaterPhysicsAssetOverride> Overrides;
};
