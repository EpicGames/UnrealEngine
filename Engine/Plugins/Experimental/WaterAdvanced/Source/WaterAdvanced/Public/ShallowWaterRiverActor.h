// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "BakedShallowWaterSimulationComponent.h"
#include "Math/Float16Color.h"
#include "NiagaraSystem.h"
#include "UObject/GCObject.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "ShallowWaterRiverActor.generated.h"

#define UE_API WATERADVANCED_API

class UNiagaraComponent;
class UNiagaraSystem;
class AWaterBody;
class AWaterZone;
class ALandscape;

UENUM(BlueprintType)
enum EShallowWaterRenderState : int
{
	WaterComponent,
	WaterComponentWithBakedSim,
	LiveSim,
	BakedSim,
	DebugRenderBottomContour,
	DebugRenderFoam,
};


UCLASS(MinimalAPI, BlueprintType, HideCategories = (Physics, Replication, Input, Collision))
class UShallowWaterRiverComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Niagara River Simulation"))
	TObjectPtr <class UNiagaraSystem> NiagaraRiverSimulation;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Resolution Max Axis"))
	int ResolutionMaxAxis;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Source Width"))
	float SourceSize;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Speed"))
	float SimSpeed = 10.f;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Num Substeps"))
	int NumSteps = 10;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Match Spline"))
	bool bMatchSpline = true;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Match Spline Height Amount", EditCondition = "bMatchSpline"))
	float MatchSplineHeightAmount = 2.;

	UPROPERTY(EditAnywhere, Category = "Simulation", meta = (DisplayName = "Remove Outside Spline Amount", EditCondition = "bMatchSpline"))
	float RemoveOutsideSplineAmount = 50.;

	UPROPERTY(EditAnywhere, Category = "Simulation")
	bool bDisableCPUThrottling = true;

	UPROPERTY(EditAnywhere, Category = "Water", meta = (DisplayName = "Source River Water Bodies"))
	TArray<TSoftObjectPtr<AWaterBody>> SourceRiverWaterBodies;

	UPROPERTY(EditAnywhere, Category = "Water", meta = (DisplayName = "Sink River Water Bodies"))
	TArray<TSoftObjectPtr<AWaterBody>> SinkRiverWaterBodies;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Render State"))
	TEnumAsByte<EShallowWaterRenderState> RenderState = EShallowWaterRenderState::WaterComponent;
	
	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Baked Sim Material"))
	TObjectPtr <class UMaterialInstance> BakedSimMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Baked UnderWater Sim Material"))
	TObjectPtr <class UMaterialInstance> BakedSimUnderWaterMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Baked Sim River To Lake Transition Material"))
	TObjectPtr <class UMaterialInstance> BakedSimRiverToLakeTransitionMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Baked Sim River To Ocean Transition Material"))
	TObjectPtr <class UMaterialInstance> BakedSimRiverToOceanTransitionMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Spline River Material"))
	TObjectPtr <class UMaterialInstance> SplineRiverMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Spline River To Lake Transition Material"))
	TObjectPtr <class UMaterialInstance> SplineRiverToLakeTransitionMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Spline River To Ocean Transition Material"))
	TObjectPtr <class UMaterialInstance> SplineRiverToOceanTransitionMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Spline River UnderWater Material"))
	TObjectPtr <class UMaterialInstance> SplineRiverUnderWaterMaterial;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Smoothing Width"))
	int SmoothingWidth = 5;

	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (DisplayName = "Smoothing Cutoff"))
	float SmoothingCutoff = 500.f;

	UPROPERTY()
	bool bUseVirtualTextures = true;

	UPROPERTY()
	FVector2D SimRes;

	UPROPERTY()
	float  SimDx;
	
	UPROPERTY()
	TObjectPtr<UTexture2D> BakedWaterSurfaceTexture;
	
	UPROPERTY()
	TObjectPtr<UTexture2D> BakedFoamTexture;

	UPROPERTY()
	TObjectPtr<UTexture2D> BakedWaterSurfaceNormalTexture;

	//UPROPERTY(EditAnywhere, Category = "Shallow Water")
	//TObjectPtr<UTexture2D> SignedDistanceToSplineTexture;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	bool bUseCapture = true;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	bool bRecursivelyAddAttachedActors = false;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<TSoftObjectPtr<AActor>> BottomContourLandscapeActors;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<TSoftObjectPtr<AActor>> BottomContourActors;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<FName> BottomContourTags;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	float BottomContourCaptureOffset = 15000.f;

	UPROPERTY(EditAnywhere, Category = "Collisions", meta = (DisplayName = "Scene Capture LOD Distance Factor"))
	float SceneCaptureLODDistanceFactor = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<TSoftObjectPtr<AActor>> DilatedBottomContourActors;

	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<FName> DilatedBottomContourTags;		

	UPROPERTY(EditAnywhere, Category = "Collisions")
	float BottomContourCollisionDilation = 0.f;

	UE_API virtual void PostLoad() override;

	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void BeginPlay() override;

	UE_API virtual void OnUnregister() override;

	UE_API void UpdateRenderState();

	UE_API void SetWaterMIDParameters(UMaterialInstanceDynamic* WaterMID);

#if WITH_EDITOR
	UE_API void Rebuild();

	UE_API void AddActorsToRawArray(const TArray<TSoftObjectPtr<AActor>>& ActorsArray, TArray<AActor*>& BottomContourActorsRawPtr);

	UE_API void AddTaggedActorsToArray(TArray<FName>& TagsToUse, TArray<AActor*>& BottomContourActorsRawPtr);

	UE_API void Bake();

	UE_API void InitializeVirtualTexture(TObjectPtr<UTexture2D> InTexture);	

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UFUNCTION()
	UE_API void OnWaterInfoTextureArrayCreated(const UTextureRenderTarget2DArray* InWaterInfoTexture);

	bool ShouldDisableCPUThrottling() { return true; }
#endif // WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = "Shallow Water River")
	UE_API void SetPaused(bool Pause);

protected:
	// Asset can be set in Project Settings - Plugins - Water ShallowWaterSimulation
	UPROPERTY(BlueprintReadOnly, VisibleDefaultsOnly, Category = "Shallow Water")
	TObjectPtr<UNiagaraComponent> RiverSimSystem;

	UPROPERTY(BlueprintReadOnly, VisibleDefaultsOnly, Category = "Shallow Water")
	TObjectPtr<const UTextureRenderTarget2DArray> WaterInfoTexture;

	UPROPERTY(BlueprintReadOnly, VisibleDefaultsOnly, Category = "Shallow Water")
	TObjectPtr<UTextureRenderTarget2D> BakedWaterSurfaceRT;

	UPROPERTY(BlueprintReadOnly, VisibleDefaultsOnly, Category = "Shallow Water")
	TObjectPtr<UTextureRenderTarget2D> BakedFoamRT;

	UPROPERTY(BlueprintReadOnly, VisibleDefaultsOnly, Category = "Shallow Water")
	TObjectPtr<UTextureRenderTarget2D> BakedWaterSurfaceNormalRT;

	UPROPERTY(BlueprintReadOnly, VisibleDefaultsOnly, Category = "Shallow Water")
	TObjectPtr<UBakedShallowWaterSimulationComponent> BakedSim;

	UE_API bool QueryWaterAtSplinePoint(TSoftObjectPtr<AWaterBody> WaterBody, int SplinePoint, FVector& OutPos, FVector& OutTangent, float& OutWidth, float& OutDepth);
	UE_API void ConvertToVirtualTextures();

private:
	bool bIsInitialized;	
	bool bTickInitialize;
	bool bRenderStateTickInitialize;

	UPROPERTY()
	TSet <TSoftObjectPtr<AWaterBody>> AllWaterBodies;

	UPROPERTY()
	FVector2D WorldGridSize;

	UPROPERTY()
	FVector SystemPos;
	
	UPROPERTY()
	TObjectPtr<UTextureRenderTarget2D> NormalDetailRT;

	UPROPERTY()
	TObjectPtr<class UNiagaraSystem> OceanPatchSystem;

	FDelegateHandle ShouldDisableCPUThrottlingDelegateHandle;

	// Find/create the level set renderer singleton actor as required. Return whether the found or created actor.
	TObjectPtr<UTextureRenderTarget2D> GetSharedFFTOceanPatchNormalRTFromSubsystem(UWorld* World);

	FBoxSphereBounds InitializeCaptureDI(const FName &DIName, TArray<AActor*> RawActorPtrArray);
	
	
};

UCLASS(MinimalAPI, BlueprintType, HideCategories = (Physics, Replication, Input, Collision))
class AShallowWaterRiver : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	// Asset can be set in Project Settings - Plugins - Water ShallowWaterSimulation
	UPROPERTY(VisibleAnywhere, Category = "Shallow Water")
	TObjectPtr<UShallowWaterRiverComponent> ShallowWaterRiverComponent;	
};

#undef UE_API
