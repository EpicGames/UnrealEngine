// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraDataChannelPublic.h"
#include "ShallowWaterCommon.h"
#include "ShallowWaterPhysicsAssetOverridesDataAsset.h"
#include "Materials/MaterialParameterCollection.h"
#include "ShallowWaterSettings.generated.h"

#define UE_API WATERADVANCED_API

UCLASS(MinimalAPI, config = Engine, defaultconfig, meta=(DisplayName="Water Advanced"))
class UShallowWaterSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	UShallowWaterSettings();
	
public:
	UE_API virtual FName GetCategoryName() const override;
	
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	TSoftObjectPtr<UNiagaraSystem> DefaultShallowWaterNiagaraSimulation;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category = "Shallow Water Simulation")
	TSoftObjectPtr<UNiagaraDataChannelAsset> DefaultShallowWaterCollisionNDC;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water River")
	TSoftObjectPtr<UNiagaraSystem> DefaultOceanPatchNiagaraSystem;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FShallowWaterSimParameters ShallowWaterSimParameters;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	TSoftObjectPtr<UMaterialParameterCollection> WaterMPC;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName GridCenterMPCName = FName("SimLocation");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName WorldGridSizeMPCName = FName("FluidSimSize");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName ResolutionMaxAxisMPCName = FName("FluidSimResolution");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName NormalRTMaterialName = FName("NormalAndHeight");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName VelocityRTMaterialName = FName("Velocity");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName NormalRTNiagaraName = FName("NormalRT");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Shallow Water Simulation")
	FName VelocityRTNiagaraName = FName("VelocityRT");
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category = "Shallow Water Simulation")
	bool UseDefaultShallowWaterSubsystem = false;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category = "Shallow Water Simulation")
	bool OutputVelocity = false;

	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Interaction")
	int32 MaxActivePawnNum = 6;
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Interaction")
	int32 MaxImpulseForceNum = 6;
	// Default overrides. Game Feature Plugin can register their own overrides asset with subsystem RegisterPhysicsAssetOverridesDataAsset()
	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Interaction")
	TSoftObjectPtr<UShallowWaterPhysicsAssetOverridesDataAsset> PhysicsAssetProxiesDataAsset;

	UPROPERTY(Config, BlueprintReadOnly, EditAnywhere, Category="Debug")
	bool bVisualizeActivePawn;
};

#undef UE_API
