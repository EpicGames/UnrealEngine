// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayTagContainer.h"
#include "ShallowWaterSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Subsystems/WorldSubsystem.h"
#include "GameplayTagContainer.h"

#include "ShallowWaterSubsystem.generated.h"

#define UE_API WATERADVANCED_API

class AWaterBody;
class AWaterZone;
class UWaterBodyComponent;
class UNiagaraComponent;
class APawn;
struct FGameplayTag;

UENUM(BlueprintType)
enum class EShallowWaterCollisionContextType : uint8
{
	Pawn,
	Vehicle, // Pawn as driver or pawn as passenger
	Custom
};

// Shallow Water Rigid Body collision
USTRUCT()
struct FShallowWaterCollisionContext
{
	GENERATED_BODY()
	
	FShallowWaterCollisionContext() {};
	FShallowWaterCollisionContext(EShallowWaterCollisionContextType InType, USkeletalMeshComponent* InComponent)
		: Type(InType)
		, WeakComponent(InComponent)
	{
		if(InComponent == nullptr)
		{
			ensureMsgf(false, TEXT("FShallowWaterCollisionContext constructor was given nullptr component as input"));
			return;
		}
		UniqueID = InComponent->GetUniqueID();
	}

	EShallowWaterCollisionContextType Type = EShallowWaterCollisionContextType::Pawn;

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshComponent> WeakComponent = nullptr;

	// #todo Theoretically not enough as unique identification since Component->GetUniqueID() is "reused so it is only unique while the object is alive"
	uint32 UniqueID = 0;

	bool IsValidAndAlive() const { return WeakComponent.Get() != nullptr; }

	bool operator==(const FShallowWaterCollisionContext& Other) const
	{
		return UniqueID == Other.UniqueID;
	}
	friend uint32 GetTypeHash(const FShallowWaterCollisionContext& Context)
	{
		return Context.UniqueID;
	}
};

/*
 * CollisionTracker that records the actor affecting the waterbody, used by logs or fishing lures etc.,
 * where the collision is handled by Niagara Data Channel and the subsystem isn't aware of them without a tracking device. This is intended as a solution.
 */
struct FShallowWaterCollisionTracker_Actor
{
	FShallowWaterCollisionTracker_Actor() {}
	FShallowWaterCollisionTracker_Actor(float InTimeSpawned, float InLifespan, TWeakObjectPtr<AActor> InCollisionActor)
		: TimeSpawned(InTimeSpawned), Lifespan(InLifespan), CollisionActor(InCollisionActor)
	{}

	UE_API void GetOverlappingWaterBodies(TSet<AWaterBody*>& WaterBodies) const;
	bool IsValid(const float CurrentTime) const { return CollisionActor.IsValid() && CurrentTime - TimeSpawned <= Lifespan; }

	float TimeSpawned = 0;
	float Lifespan = 10.f;
	TWeakObjectPtr<AActor> CollisionActor = nullptr;

	bool operator==(const FShallowWaterCollisionTracker_Actor& Other) const
	{
		return CollisionActor == Other.CollisionActor;
	}
};

/*
 * CollisionTracker that directly records the water body affected, used by Impacts
 */
struct FShallowWaterCollisionTracker_Direct
{
	FShallowWaterCollisionTracker_Direct() {}
	FShallowWaterCollisionTracker_Direct(float InTimeSpawned, float InLifespan, TWeakObjectPtr<AWaterBody> InWaterBody)
		: TimeSpawned(InTimeSpawned), Lifespan(InLifespan), WaterBody(InWaterBody)
	{}

	AWaterBody* GetOverlappingWaterBody() const { return WaterBody.Get(); }
	bool IsValid(const float CurrentTime) const { return WaterBody.IsValid() && CurrentTime - TimeSpawned <= Lifespan; }

	float TimeSpawned = 0;
	float Lifespan = 10.f;
	TWeakObjectPtr<AWaterBody> WaterBody = nullptr;
};


UCLASS(MinimalAPI, Abstract)
class UShallowWaterSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UE_API UShallowWaterSubsystem();	
	UE_API virtual void PostInitialize() override;
	UE_API virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;

	UE_API virtual void InitializeShallowWater();
	/*
	 * Note: A world subsystem is always created and activated even when the plugin (GFP or not) it resides in is completely disabled
	 * When the subclass of this class is a Game Feature Plugin. This can be used to limit when ShallowWater is actually enabled
	 * by checking GameFeaturesSubsystem.GetPluginURLByName(PluginName, PluginURL))
	 * and GameFeaturesSubsystem.IsGameFeaturePluginActive(PluginURL, true)
	 */ 
	UE_API virtual bool IsShallowWaterAllowedToInitialize() const;
	UE_API bool IsShallowWaterInitialized() const;
	UE_API APawn* GetNonSpectatorPawnFromWeakController() const;
	UE_API TOptional<FVector> GetCameraLocationFromWeakController() const;
	
	// Think of this as a cursor that in most time locks on the current player pawn
	// If the player doesn't have a physical pawn (e.g. spectating), the cursor jump around and lock on the nearest pawn to the camera
	// If there is no relevant pawn (all eliminate), returns nullptr
	UE_API APawn* GetTheMostRelevantPlayerPawn() const;
	virtual FGameplayTagContainer GetVehicleTags(FShallowWaterCollisionContext Context) const { return FGameplayTagContainer::EmptyContainer; }

	UE_API void CreateRTs();
	UE_API void InitializeParameters();
	/*
	 * @param WaterBody  
	 */
	UE_API void UpdateGridMovement();

	UFUNCTION(BlueprintCallable, Category = "Shallow Water")
	UE_API void RegisterImpact(FVector ImpactPosition, FVector ImpactVelocity, float ImpactRadius);
	
	UE_API void FlushPendingImpacts();
	UE_API void WriteImpactToNDC(FVector ImpactPosition, FVector ImpactVelocity, float ImpactRadius);
	// Override to return the ECC channel of bullets
	virtual ECollisionChannel GetImpactCollisionChannel() { return ECC_WorldDynamic; }
	

	// Manually set MID parameters for water bodies before they collide with any pawns or CollisionTrackers 
	UFUNCTION(BlueprintCallable, Category = "Shallow Water")
	UE_API void SetWaterBodyMIDParameters(AWaterBody* WaterBody);

	UE_API void TryUpdateWaterBodyMIDParameters(UWaterBodyComponent* WaterBodyComponent);
	float GetGridSize() const { return Settings->ShallowWaterSimParameters.WorldGridSize; }
	int32 GetGridResolution() const { return Settings->ShallowWaterSimParameters.ResolutionMaxAxis; }	

	/*
	 * Add PA overrides. Designed to be called by Game Feature Plugins. 
	 */
	UE_API void RegisterPhysicsAssetProxiesDataAsset(const UShallowWaterPhysicsAssetOverridesDataAsset* Proxies);

protected:
	// Asset can be set in Project Settings - Plugins - Water ShallowWaterSimulation
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TObjectPtr<UNiagaraComponent> ShallowWaterNiagaraSimulation;

	/*
	 * WaterBody is used for:
	 *     Get water data texture
	 *     Get water zone
	 *     Get character location projected onto water surface
	 *     Check if character or vehicle is 'in water'
	 * @return Should be the the water body touched by the most relevant pawn. If that's not available we try to find water body touched by nearby pawns, sorted by significance.
	 */
	UFUNCTION(BlueprintCallable, Category="Shallow Water")
	UE_API virtual TSet<AWaterBody*> GetAllOverlappingWaterBodiesAndUpdateCollisionTrackers();
	/*
	 * @param MaxLifespan    Max lifespan in case RemoveCollisionTrackerForActor() is not called on tracker destroy, which could causing the sim to keep active indefinitely 
	 */
	UFUNCTION(BlueprintCallable, Category="Shallow Water")
	UE_API void AddCollisionTrackerForActor(AActor* CollisionTrackerActor, float MaxLifespan = 60.f);
	UFUNCTION(BlueprintCallable, Category="Shallow Water")
	UE_API void RemoveCollisionTrackerForActor(AActor* CollisionTrackerActor);
	
	UE_API virtual TSet<AWaterBody*> GetOverlappingWaterBodiesFromPawns() const;
	UE_API void GetOverlappingWaterBodiesFromActorTrackersAndUpdate(TSet<AWaterBody*>& WaterBodies);
	UE_API void GetOverlappingWaterBodiesFromDirectTrackersAndUpdate(TSet<AWaterBody*>& WaterBodies);
	UE_API void UpdateOverlappingWaterBodiesHistory(TArray<AWaterBody*> OverlappingWaterBodies);
	UE_API bool ShouldSimulateThisFrame() const;
	UE_API void ClearTickCache();

	/*
	 * Collision
	 */

	static UE_API FName ColliderComponentTag;
	
	// Override to get most relevant pawns every frame, this also sets up the per-tick cache that is reset with ClearTickCache
	UE_API virtual int32 UpdateActivePawns();
	// By default getting the first SKM, if is ACharacter, get 'Mesh' component
	// Override if e.g. The pawn is driving a boat, return SKM of boat instead
	UE_API virtual TOptional<FShallowWaterCollisionContext> GetCollisionContextFromPawn(APawn* InPawn) const;
	// Remove invalid references. The owning actor may get destroyed.
	UE_API void CleanUpVehicleCollisionProxies();
	UE_API void UpdateCollisionForPendingContexts();
	UE_API void EnableCollisionForContext(const FShallowWaterCollisionContext& Context);
	UE_API void DisableCollisionForContext(const FShallowWaterCollisionContext& Context);
	// VehicleCollisionProxies is handled inside
	UE_API void DisableCollisionForVehicle(const FShallowWaterCollisionContext& Context);

	UE_API virtual float GetColliderMaxRange() const;
	// For overriden functions: Do not use GetTheMostRelevantPlayerPawn() inside to avoid loop. Use local controlled pawn location or camera location instead
	UE_API virtual TArray<APawn*> GetPawnsInRange(const bool bShouldSortBySignificance = false) const;
	UE_API virtual TArray<APawn*> GetPawnsInRange(const FVector ObservingLocation, const bool bShouldSortBySignificance = false) const;

	/*
	 * WaterBody
	 */
	UE_API void SetSimParametersForWaterZone(AWaterZone* WaterZone);
	UE_API bool TryGetOrWaitForWaterInfoTextureFromWaterBody(AWaterBody* CurrentWaterBody);
	UE_API void TryGetOrWaitForWaterInfoTextureFromWaterBodies(TSet<AWaterBody*> CurrentWaterBodies);
	UFUNCTION()
	UE_API void OnWaterInfoTextureArrayCreated(const UTextureRenderTarget2DArray* WaterInfoTexture);

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TWeakObjectPtr<const UTextureRenderTarget2DArray> WaterInfoTexture;
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TObjectPtr<UTextureRenderTarget2D> NormalRT;
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TObjectPtr<UTextureRenderTarget2D> VelocityRT;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TObjectPtr<UShallowWaterSettings> Settings;
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Shallow Water")
	TObjectPtr<UMaterialParameterCollection> MPC;

	// Consistent record through multiple frames since we want active pawns that affects the fluidsim to remain active
	// Even when it's less significant than newly available pawns. Fluidsim enabled for different pawns each frame is bad.
	TArray<TWeakObjectPtr<APawn>> ActivePawns;

	FVector PreviousProjectedLocation = FVector::ZeroVector;
	TWeakObjectPtr<APlayerController> WeakPlayerController;

	// Cached variables only valid during a tick
	bool bTickCacheValid = false;
	APawn* CachedCursorPawn = nullptr;
	TArray<APawn*> CachedPawnsInRange;
	TWeakObjectPtr<AWaterBody> BestWaterBody;

private:
	float LastTimeOverlappingAnyWaterBody = -FLT_MAX;
	// Should only be managed by UpdateOverlappingWaterBodiesHistory()
	TArray<TWeakObjectPtr<AWaterBody>> LastOverlappingWaterBodies_Internal;

	bool bIsShallowWaterInitialized = false;

	bool bInitializationAsyncLoadsAttempted = false;
	
	TSet<TWeakObjectPtr<AWaterZone>> WaterZonesWithMaterialsInitialized;

	struct PendingImpact
	{
		FVector ImpactPosition;
		FVector ImpactVelocity;
		float ImpactRadius;
	};

	TArray<PendingImpact> PendingImpacts;
	bool bFlushPendingImpactsNextTick;

	TSet<TWeakObjectPtr<UWaterBodyComponent>> WaterBodyComponentsWithProperMIDParameters;
	TArray<TWeakObjectPtr<AWaterBody>> PendingWaterBodiesToSetMIDOnInitialize;

	/*
	 * Collision Context
	 */
	UPROPERTY()
	TArray<FShallowWaterCollisionContext> PreviousContexts;
	// 'Pending' also includes contexts already with collision enabled that will be skipped
	UPROPERTY()
	TArray<FShallowWaterCollisionContext> PendingContexts;
	UPROPERTY()
	TMap<FShallowWaterCollisionContext, TObjectPtr<USkeletalMeshComponent>> VehicleCollisionProxies;

	FTimerHandle WaitForPlayerControllerHandle;
	UFUNCTION()
	UE_API void OnLocalPlayerControllerBecomesValid(APlayerController* InPlayerController);
	UFUNCTION()
	UE_API void OnLocalPlayerPawnBecomesValid(APawn* OldPawn, APawn* NewPawn);

	TArray<FShallowWaterCollisionTracker_Actor> Tracker_Actors;
	TArray<FShallowWaterCollisionTracker_Direct> Tracker_Directs;

	// Overrides collected from RegisterPhysicsAssetOverridesDataAsset
	TMap<FGameplayTag, FShallowWaterPhysicsAssetOverride> RegisteredPhysicsAssetProxies;
};

#undef UE_API
