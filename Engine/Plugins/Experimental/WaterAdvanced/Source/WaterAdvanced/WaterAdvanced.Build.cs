// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WaterAdvanced : ModuleRules
{
	public WaterAdvanced(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"Niagara",
				"Water",
				"DeveloperSettings",
				"Projects",
				"GameplayTags",
				"Landscape"
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"TargetPlatform",
				"DerivedDataCache",
				"EditorFramework",
				"UnrealEd",
				"SlateCore",
				"Slate"
			});
		}


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
