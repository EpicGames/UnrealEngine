{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Water Advanced",
	"Description": "Collection of easy to use water simulation systems built on the Niagara Fluids and Water plugins",
	"Category": "Water",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WaterAdvanced",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "NiagaraFluids",
			"Enabled": true
		},
		{
			"Name": "Water",
			"Enabled": true
		}
	]
}